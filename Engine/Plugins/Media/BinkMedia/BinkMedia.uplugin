{
	"FileVersion" : 3,
	"Version" : 2,
	"VersionName" : "2.0",
	"FriendlyName" : "Bink Media",
	"Description" : "Implements a media player using Bink.",
	"Category": "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
    "Modules" :
	[
		{
			"Name" : "BinkMediaPlayer",
			"Type" : "Runtime",
			"LoadingPhase" : "PreLoadingScreen",
			"TargetDenyList": [ "Server" ],
			"PlatformDenyList": [ 
				"tvOS"
			]
		},
		{
			"Name" : "BinkMediaPlayerEditor",
			"Type" : "Editor",
			"TargetDenyList": [ "Server" ],
			"PlatformDenyList": [ 
			]
		}
    ]
}