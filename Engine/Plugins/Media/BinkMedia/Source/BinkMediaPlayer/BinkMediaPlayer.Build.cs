// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class BinkMediaPlayer : ModuleRules 
{
	public BinkMediaPlayer(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.Add("Core");
		PublicDependencyModuleNames.Add("CoreUObject");
		PublicDependencyModuleNames.Add("Engine");
		PublicDependencyModuleNames.Add("InputCore");
		PublicDependencyModuleNames.Add("MoviePlayer");
		PublicDependencyModuleNames.Add("Projects");

		PrivateDependencyModuleNames.Add("RenderCore");
		PrivateDependencyModuleNames.Add("RHI");
		PrivateDependencyModuleNames.Add("Renderer");
		PrivateDependencyModuleNames.Add("Slate");
		PrivateDependencyModuleNames.Add("SlateCore");
		
		AddEngineThirdPartyPrivateStaticDependencies(Target, "BinkMediaPlayerSDK");
		
		if (Target.bBuildEditor == true)
		{
			PublicDependencyModuleNames.Add("Slate");
			PublicDependencyModuleNames.Add("SlateCore");
			PublicDependencyModuleNames.Add("DesktopWidgets");
			PublicDefinitions.Add("BINKPLUGIN_UE4_EDITOR=1");
			PrivateDependencyModuleNames.AddRange(new string[] { "PropertyEditor", "DesktopPlatform", "SourceControl", "EditorStyle", "UnrealEd" });
		}
		else
		{
			PublicDefinitions.Add("BINKPLUGIN_UE4_EDITOR=0");
		}
		
		//RuntimeDependencies.Add("$(ProjectDir)/Content/Movies/..."); // For chunked streaming
		
		if (Target.Platform == UnrealTargetPlatform.Android)
		{
			string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
			AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "BinkMediaPlayer_APL.xml"));
		}
	}
}
