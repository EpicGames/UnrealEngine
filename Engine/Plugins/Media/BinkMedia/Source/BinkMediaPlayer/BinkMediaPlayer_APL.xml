<?xml version="1.0" encoding="utf-8"?>
<!--Bink plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="Bink init"/>
		<setBool result="bSupported" value="false"/>
		<isArch arch="armeabi-v7a">
			<setBool result="bSupported" value="true"/>
		</isArch>
		<isArch arch="arm64-v8a">
			<setBool result="bSupported" value="true"/>
		</isArch>
	</init>
</root>
