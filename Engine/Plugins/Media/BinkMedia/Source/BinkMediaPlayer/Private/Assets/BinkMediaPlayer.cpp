// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkMediaPlayer.h"

#include "BinkMediaPlayerPrivate.h"

#include "BinkMovieStreamer.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Texture.h"
#include "Misc/Paths.h"
#include "RenderingThread.h"

#include "HAL/PlatformFileManager.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"
#include "SubtitleManager.h"
#include "UObject/UObjectIterator.h"
#include "BinkUE.h"
#include "RectLightTexture.h"

static void Command_ShowBinks()
{
	for (TObjectIterator<UBinkMediaPlayer> It; It; ++It)
	{
		if (!It->HasAnyFlags(RF_ClassDefaultObject))
		{
			const TCHAR* StateStr = TEXT("Unloaded");
			if (It->IsInitialized())
			{
				if (It->IsPaused())
				{
					StateStr = TEXT("Paused");
				}
				else
				{
					StateStr = TEXT("Playing");
				}
			}

			const TCHAR* DrawStyleStr = TEXT("RenderToTexture");
			if (It->BinkDrawStyle != BMASM_Bink_DS_RenderToTexture)
			{
				DrawStyleStr = TEXT("Overlay");
			}

			const double DurationSeconds = It->GetDuration().GetTotalSeconds();
			const double PlaybackCompletion = (DurationSeconds > 0.0) ? It->GetTime().GetTotalSeconds() / DurationSeconds : 0.0;

			UE_LOGF(LogBinkMoviePlayer, Display, "[%ls] URL: %ls, DrawStyle: %ls, State: %ls, PlaybackPercentage: %d%%", *It->GetName(), *It->GetUrl(), DrawStyleStr, StateStr, static_cast<int>(PlaybackCompletion * 100.f));
		}
	}
}

static FAutoConsoleCommand ConsoleCmdShowBinks(
	TEXT("bink.List"),
	TEXT("Display all Bink Media Player objects and their current state"),
	FConsoleCommandDelegate::CreateStatic(&Command_ShowBinks));

UBinkMediaPlayer::UBinkMediaPlayer( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, FTickableGameObject(ETickableTickType::Never)
	, Looping(true)
	, StartImmediately(true)
	, DelayedOpen(true)
	, BinkDestinationUpperLeft(0,0)
	, BinkDestinationLowerRight(1,1)
	, BinkBufferMode(BMASM_Bink_Stream)
	, BinkSoundTrack(BMASM_Bink_Sound_None)
	, BinkSoundTrackStart(0)
	, BinkDrawStyle()
	, BinkLayerDepth()
	, CurrentBinkBufferMode(BMASM_Bink_MAX)
	, CurrentBinkSoundTrack(BMASM_Bink_Sound_MAX)
	, CurrentBinkSoundTrackStart(-1)
	, CurrentUrl()
	, CurrentDrawStyle()
	, CurrentLayerDepth()
    , bnk()
    , bink_textures()
    , paused()
    , reached_end()
{
}

bool UBinkMediaPlayer::CanPause() const { return IsPlaying(); }
bool UBinkMediaPlayer::CanPlay() const { return IsReady(); }
bool UBinkMediaPlayer::IsStopped() const { return !IsReady(); }
const FString& UBinkMediaPlayer::GetUrl() const { return CurrentUrl; }
bool UBinkMediaPlayer::Pause() { return SetRate(0.0f); }
bool UBinkMediaPlayer::Play() { return SetRate(1.0f); }
bool UBinkMediaPlayer::Rewind() { return Seek(FTimespan::Zero()); }
bool UBinkMediaPlayer::SupportsRate( float Rate, bool Unthinned ) const { return Rate == 1; }
bool UBinkMediaPlayer::SupportsScrubbing() const { return true; }
bool UBinkMediaPlayer::SupportsSeeking() const { return true; }
FString UBinkMediaPlayer::GetDesc() { return TEXT("UBinkMediaPlayer"); }

FTimespan UBinkMediaPlayer::GetDuration() const 
{
	double ms = 0;
	if(bnk) 
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		ms = ((double)info.Frames) * ((double)info.FrameRateDiv) * 1000.0 / ((double)info.FrameRate);
	}
	return FTimespan::FromMilliseconds(ms);
}

float UBinkMediaPlayer::GetRate() const 
{
	if(bnk) 
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		return info.PlaybackState == 0 && !paused ? 1 : 0;
	}
	return 0; 
}

FTimespan UBinkMediaPlayer::GetTime() const 
{
	double ms = 0;
	if(bnk) 
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		ms = ((double)info.FrameNum) * ((double)info.FrameRateDiv) * 1000.0 / ((double)info.FrameRate);
	}
	return FTimespan::FromMilliseconds(ms);
}

bool UBinkMediaPlayer::IsLooping() const 
{ 
	if(bnk) 
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		if (info.PlaybackState < 3 && !info.LoopsRemaining) 
		{
			return true;
		}
	}
	return false;
}

bool UBinkMediaPlayer::IsPaused() const 
{ 
	if(bnk) 
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		return info.PlaybackState == 1 || paused;
	}
	return false;
}

bool UBinkMediaPlayer::IsPlaying() const 
{
	if(bnk) 
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		return info.PlaybackState == 0 && !paused;
	}
	return false;
}

bool UBinkMediaPlayer::IsGotoing() const
{
	if (bnk)
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		return info.PlaybackState == 2;
	}
	return false;
}

bool UBinkMediaPlayer::OpenUrl( const FString& NewUrl ) 
{
	if (NewUrl.IsEmpty()) 
	{
		return false;
	}

	URL = NewUrl;
	InitializePlayer();
	return CurrentUrl == NewUrl;
}

void UBinkMediaPlayer::CloseUrl( ) 
{
	URL = "";
	InitializePlayer();
}

bool UBinkMediaPlayer::SetLooping( bool InLooping ) 
{
	if(bnk) 
	{
		// 0 = loop forever, 1 = play once
		BinkHLLoopCount(bnk, InLooping ? 0 : 1);
	}
	return false;
}

bool UBinkMediaPlayer::SetRate( float Rate ) 
{ 
	if(bnk) 
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		if ((info.PlaybackState == 1 || paused) && Rate == 1) 
		{
			BinkHLPause(bnk, 0);
			paused      = false;
			reached_end = false;
			return true;
		} 
		else if (info.PlaybackState == 3 && Rate == 1) 
		{
			// Rewind to start after reaching end
			BinkHLGoto(bnk, 1, 4);
			BinkHLPause(bnk, 0);
			paused      = false;
			reached_end = false;
			return true;
		} 
		else if (info.PlaybackState == 0 && Rate == 0) 
		{
			BinkHLPause(bnk, 1);
			paused = true;
			OnPlaybackSuspended.Broadcast();
			return true;
		}
	}
	return false;
}

void UBinkMediaPlayer::SetVolume( float Volume ) 
{ 
	if(bnk) 
	{
		Volume = Volume < 0 ? 0 : Volume > 1 ? 1 : Volume;
		BinkHLVolume(bnk, Volume);
	}
}

bool UBinkMediaPlayer::Seek( const FTimespan& InTime ) 
{ 
	if (!bnk) 
	{
		return false;
	}
	BINKHLINFO info = {};
	BinkHLGetInfo(bnk, &info);
	U32 desiredFrame = (U32)(InTime.GetTotalMilliseconds() * ((double)info.FrameRate) / (1000.0 * ((double)info.FrameRateDiv)));
	if (info.FrameNum != (desiredFrame + 1))
	{
		BinkHLGoto(bnk, desiredFrame + 1, 4);
		reached_end = false;
	}
	return true;
}

void UBinkMediaPlayer::BeginDestroy() 
{
	Super::BeginDestroy();
	Close();
}

void UBinkMediaPlayer::PostLoad() 
{
	Super::PostLoad();
	if (!HasAnyFlags(RF_ClassDefaultObject) && !GIsBuildMachine && (!DelayedOpen || StartImmediately)) 
	{
		InitializePlayer();
	}
}

#if BINKPLUGIN_UE4_EDITOR
void UBinkMediaPlayer::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent ) 
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	InitializePlayer();
}
#endif

bool UBinkMediaPlayer::Open(const FString& Url) 
{
	if (!BinkInitialize())
	{
		UE_LOGF(LogBinkMoviePlayer, Error, "UBinkMediaPlayer::Open: failed to initialize bink!");
		return false;
	}

	if (Url.IsEmpty())
	{
		UE_LOGF(LogBinkMoviePlayer, Error, "UBinkMediaPlayer::Open: Failed! Url is empty.");
		return false;
	}

	if (IsPlaying()) 
	{
		UE_LOGF(LogBinkMoviePlayer, Error, "UBinkMediaPlayer::Open: Failed! Already playing.");
		return false;
	}

	if(bnk) 
	{
		BinkCloseOnRenderThread(bnk, bink_textures);
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

#if PLATFORM_ANDROID
	FString MoviePath = Url;
	FPaths::NormalizeFilename(MoviePath);

	if (!IAndroidPlatformFile::GetPlatformPhysical().FileExists(*MoviePath)) {
		UE_LOGF(LogBinkMoviePlayer, Error, "UBinkMediaPlayer::Open: Failed! File doesn't exist. URL :%ls File :%ls", *Url, *MoviePath);
		return false;
	}

	int64 FileOffset      = IAndroidPlatformFile::GetPlatformPhysical().FileStartOffset(*MoviePath);
	FString FileRootPath  = IAndroidPlatformFile::GetPlatformPhysical().FileRootPath(*MoviePath);

	if (IAndroidPlatformFile::GetPlatformPhysical().IsAsset(*MoviePath)) 
	{
		FString APKPath = BinkGetAndroidAPKPath();
		if (!APKPath.IsEmpty())
		{
			bnk = BinkHLOpen(TCHAR_TO_UTF8(*APKPath), (U32)BinkSoundTrack, (U32)BinkSoundTrackStart, (U32)BinkBufferMode, (U64)FileOffset);
		}
	}
	else 
	{
		bnk = BinkHLOpen(TCHAR_TO_UTF8(*FileRootPath), (U32)BinkSoundTrack, (U32)BinkSoundTrackStart, (U32)BinkBufferMode, (U64)FileOffset);
	}
#else
	if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*Url)) 
	{
		FString AbsPath = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*Url);
		bnk = BinkHLOpen(TCHAR_TO_UTF8(*AbsPath), (U32)BinkSoundTrack, (U32)BinkSoundTrackStart, (U32)BinkBufferMode, 0);
	}
#endif

	if(!bnk) 
	{
		UE_LOGF(LogBinkMoviePlayer, Error, "UBinkMediaPlayer::Open: Failed! BinkHLOpen failed. URL: %ls Error: %ls", *Url, UTF8_TO_TCHAR(BinkGetError()));
		return false;
	}

	// Create CPU-side decode buffers immediately on the game thread.
	// GPU textures are created lazily on the render thread on first Draw_textures call.
	bink_textures = FBinkTextures::CreateCPU(bnk, nullptr);

	if (!bink_textures)
	{
		UE_LOGF(LogBinkMoviePlayer, Error, "UBinkMediaPlayer::Open: Failed to create CPU decode buffers. URL: %ls", *Url);
		BinkCloseOnRenderThread(bnk, bink_textures);
		return false;
	}

	paused      = false;
	reached_end = false;
	CurrentHasSubtitles = 0;

	// Search for a language-matched .srt file and load it via BinkLoadSubtitles.
	{
		IPlatformFile& PF = FPlatformFileManager::Get().GetPlatformFile();
		FInternationalization& I18N = FInternationalization::Get();
		const TArray<FString> LangNames = I18N.GetPrioritizedCultureNames(I18N.GetCurrentLanguage()->GetName());

		for (const FString& LangName : LangNames)
		{
			int32 DotPos = INDEX_NONE;
			if (Url.FindLastChar(TEXT('.'), DotPos))
			{
				const int32 SlashPos = Url.FindLastCharByPredicate([](TCHAR C){ return C == TEXT('/') || C == TEXT('\\'); });
				if (SlashPos != INDEX_NONE && SlashPos > DotPos)
				{
					DotPos = INDEX_NONE;
				}
			}
			FString SubUrl = (DotPos == INDEX_NONE ? Url : Url.Left(DotPos)) + TEXT("_") + LangName + TEXT(".srt");

			if (PF.FileExists(*SubUrl))
			{
				FString AbsSub = PF.ConvertToAbsolutePathForExternalAppForRead(*SubUrl);
				if (BinkLoadSubtitles(bnk, TCHAR_TO_UTF8(*AbsSub)))
				{
					CurrentHasSubtitles = 1;
					break;
				}
			}
		}
	}

	SetTickableTickType(ETickableTickType::Conditional);
	HandleMediaPlayerMediaOpened(Url);
	return true;
}

void UBinkMediaPlayer::Close() 
{
	SetTickableTickType(ETickableTickType::Never);
	if(bnk) 
	{
		if (CurrentHasSubtitles)
		{
			CurrentHasSubtitles = 0;
			TArray<FString> Empty = { TEXT("") };
			FSubtitleManager::GetSubtitleManager()->SetMovieSubtitle(this, Empty);
		}

		BinkCloseOnRenderThread(bnk, bink_textures);
	}
	CurrentUrl = FString();
	HandleMediaPlayerMediaClosed();
}

void UBinkMediaPlayer::InitializePlayer() 
{
	if (URL != CurrentUrl 
		|| BinkBufferMode != CurrentBinkBufferMode 
		|| BinkSoundTrack != CurrentBinkSoundTrack 
		|| BinkSoundTrackStart != CurrentBinkSoundTrackStart 
		|| BinkDrawStyle != CurrentDrawStyle
		|| BinkLayerDepth != CurrentLayerDepth ) 
	{
		Close();

		if (URL.IsEmpty()) 
		{
			return;
		}

		bool OpenedSuccessfully = false;

		FString FullUrl = FPaths::ConvertRelativePathToFull(FPaths::IsRelative(URL) ? BINKCONTENTPATH / URL : URL);
		OpenedSuccessfully = Open(FullUrl);
		if (!OpenedSuccessfully) 
		{
			FString cookpath = BinkUE4CookOnTheFlyPath(FPaths::ConvertRelativePathToFull(BINKCONTENTPATH), *URL);
			OpenedSuccessfully = Open(cookpath);
		}

		if (OpenedSuccessfully) 
		{
			CurrentUrl              = URL;
			CurrentBinkBufferMode   = BinkBufferMode;
			CurrentBinkSoundTrack   = BinkSoundTrack;
			CurrentBinkSoundTrackStart = BinkSoundTrackStart;
			CurrentDrawStyle        = BinkDrawStyle;
			CurrentLayerDepth       = BinkLayerDepth;
		}
	}

	SetLooping(Looping);
	SetRate(StartImmediately ? 1 : 0);
}

void UBinkMediaPlayer::HandleMediaPlayerMediaClosed() 
{
	MediaChangedEvent.Broadcast();
	OnMediaClosed.Broadcast();
}

void UBinkMediaPlayer::HandleMediaPlayerMediaOpened( FString OpenedUrl ) 
{
	MediaChangedEvent.Broadcast();
	OnMediaOpened.Broadcast(OpenedUrl);
}

void UBinkMediaPlayer::Tick(float DeltaTime) 
{
	// Fire the reached-end event once when playback stops at the end
	if (bnk && !reached_end && !IsPlaying() && !IsGotoing() && !IsPaused() && !IsLooping()) 
	{
		OnMediaReachedEnd.Broadcast();
		reached_end = true;
	}

	// Update subtitles each tick using the native BinkCurrentSubtitle iterator.
	if (bnk && CurrentHasSubtitles)
	{
		TArray<FString> SubtitlesText;
		U32 iterate = 0;
		while (const char* sub = BinkCurrentSubtitle(bnk, &iterate, nullptr, nullptr))
		{
			SubtitlesText.Add(FString(UTF8_TO_TCHAR(sub)));
		}
		FSubtitleManager::GetSubtitleManager()->SetMovieSubtitle(this, SubtitlesText);
	}

	// Overlay draw style: schedule a draw to the backbuffer each frame
	if (bnk && BinkDrawStyle != 0 && GEngine && GEngine->GameViewport) 
	{
		if (!IsPlaying() && !IsPaused()) 
		{
			return;
		}

		FVector2D screenSize;
		GEngine->GameViewport->GetViewportSize(screenSize);

		float ulx = BinkDestinationUpperLeft.X;
		float uly = BinkDestinationUpperLeft.Y;
		float lrx = BinkDestinationLowerRight.X;
		float lry = BinkDestinationLowerRight.Y;

		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		int binkw = (int)info.Width;
		int binkh = (int)info.Height;

		if (BinkDrawStyle == 1 /*OverlayFillScreenWithAspectRatio*/) 
		{
			lrx = binkw / screenSize.X;
			lry = binkh / screenSize.Y;
			if (lrx > lry) 
			{
				lry /= lrx; lrx = 1;
			}
			else 
			{
				lrx /= lry; lry = 1;
			}
			ulx = (1.0f - lrx) / 2.0f;
			uly = (1.0f - lry) / 2.0f;
			lrx += ulx;
			lry += uly;
		}
		else if (BinkDrawStyle == 2 /*OverlayOriginalMovieSize*/) 
		{
			ulx = (screenSize.X - binkw) / (2.0f * screenSize.X);
			uly = (screenSize.Y - binkh) / (2.0f * screenSize.Y);
			lrx = binkw / screenSize.X + ulx;
			lry = binkh / screenSize.Y + uly;
		}

#if PLATFORM_ANDROID
		uly = 1 - uly;
		lry = 1 - lry;
#endif

		// Bink_DrawOverlays() reads bink_textures at DrawBinks time (after all Ticks),
		// so it reflects any Close() that happened this Tick.
		PendingBinkOverlays.Add({ this, ulx, uly, lrx, lry });
	}
}

void UBinkMediaPlayer::UpdateTexture(const UTexture *tex, FRHICommandListImmediate &RHICmdList, FTextureRHIRef ref, void *nativePtr, int width, int height, bool isEditor, bool tonemap, int output_nits, float alpha, bool srgb_decode, bool is_hdr) 
{
	check(IsInRenderingThread());

	if (!bnk || !bink_textures || (!isEditor && BinkDrawStyle != 0))
	{
		return;
	}

	// Draw directly. BinkHLProcess() has already run on this frame's render thread
	// (from the DrawBinks render command which executes before UpdateDeferredResource).
	S32 draw_flags = srgb_decode ? (S32)0x80000000 : 0;

	bink_textures->SetRenderTarget(ref.GetReference(), ERenderTargetLoadAction::EClear);

	bink_textures->SetHDRSettings(tonemap, 1.0f, output_nits);
	bink_textures->SetAlphaSettings(alpha, draw_flags);
	bink_textures->SetDrawPosition(
		(float)BinkDestinationUpperLeft.X,  (float)BinkDestinationUpperLeft.Y,
		(float)BinkDestinationLowerRight.X, (float)BinkDestinationLowerRight.Y);

	bink_textures->Draw();

	if (tex)
	{
		RectLightAtlas::FAtlasTextureInvalidationScope InvalidationScope(tex);
	}
}

void UBinkMediaPlayer::Draw(UTexture *texture, bool tonemap, int out_nits, float alpha, bool srgb_decode, bool hdr) 
{
	if (!bnk || !texture->GetResource()) 
	{
		return;
	}
	FTextureRHIRef ref = texture->GetResource()->TextureRHI->GetTexture2D();
	if ((!IsPlaying() && !IsPaused()) || !ref) 
	{
		return;
	}
	int width  = texture->GetSurfaceWidth();
	int height = texture->GetSurfaceHeight();
	void *native = ref->GetNativeResource();
	if (!native) 
	{
		texture->UpdateResource();
		return;
	}

	struct parms_t 
	{
		UBinkMediaPlayer* player;
		UTexture*         tex;
		FTextureRHIRef    ref;
		void*             native;
		int               width, height;
		bool              tonemap;
		bool              srgb_decode;
		bool              hdr;
		int               out_nits;
		float             alpha;
	} parms = { this, texture, ref, native, width, height, tonemap, srgb_decode, hdr, out_nits, alpha };

	ENQUEUE_RENDER_COMMAND(BinkMediaPlayer_Draw)([parms](FRHICommandListImmediate& RHICmdList) 
	{ 
		parms.player->UpdateTexture(parms.tex, RHICmdList, parms.ref, parms.native, parms.width, parms.height, false, parms.tonemap, parms.out_nits, parms.alpha, parms.srgb_decode, parms.hdr);
	});
}

FIntPoint UBinkMediaPlayer::GetDimensions() const
{
	if (bnk)
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		return FIntPoint(info.Width, info.Height);
	}
	return FIntPoint(0, 0);
}

float UBinkMediaPlayer::GetFrameRate() const
{
	if (bnk)
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(bnk, &info);
		return (float)(((double)info.FrameRate) / ((double)info.FrameRateDiv));
	}
	return 0;
}
