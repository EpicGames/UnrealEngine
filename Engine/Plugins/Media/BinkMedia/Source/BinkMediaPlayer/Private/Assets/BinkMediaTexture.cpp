// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkMediaTexture.h"

#include "BinkMediaPlayer.h"
#include "BinkMediaPlayerPrivate.h"
#include "BinkMediaTextureResource.h"
#include "RenderingThread.h"

UBinkMediaTexture::UBinkMediaTexture( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, MediaPlayer(nullptr)
	, Tonemap(false)
	, OutputNits(10000)
	, Alpha(1)
	, CachedDimensions()
	, CurrentMediaPlayer()
	, ReleasePlayerFence()
{
	NeverStream = true;
	if (GRHISupportsHDROutput)
	{
		PixelFormat = PF_A2B10G10R10;
		DecodeSRGB  = true;
	}
	else
	{
		PixelFormat = PF_B8G8R8A8;
		DecodeSRGB  = true;
	}
}

UBinkMediaTexture::~UBinkMediaTexture() 
{
}

void UBinkMediaTexture::SetMediaPlayer( UBinkMediaPlayer* InMediaPlayer ) 
{
	MediaPlayer = InMediaPlayer;
	InitializeTrack();
}

void UBinkMediaTexture::Clear() 
{
	if (GetResource()) 
	{
		FBinkMediaTextureResource *res = (FBinkMediaTextureResource *)GetResource();
		res->Clear();
	}
}

void UBinkMediaTexture::BeginDestroy() 
{
	Super::BeginDestroy();

	// synchronize with the rendering thread by inserting a fence
 	if (!ReleasePlayerFence) {
 		ReleasePlayerFence = new FRenderCommandFence();
 	}

 	ReleasePlayerFence->BeginFence();
}

void UBinkMediaTexture::FinishDestroy() 
{
	delete ReleasePlayerFence;
	ReleasePlayerFence = nullptr;

	Super::FinishDestroy();
}

FString UBinkMediaTexture::GetDesc() 
{
	return FString::Printf(TEXT("%dx%d [%s]"), CachedDimensions.X, CachedDimensions.Y, GPixelFormats[PixelFormat].Name);
}

void UBinkMediaTexture::PostLoad() 
{
	Super::PostLoad();

	if (!HasAnyFlags(RF_ClassDefaultObject) && !GIsBuildMachine) 
	{
		InitializeTrack();
	}
}

#if BINKPLUGIN_UE4_EDITOR

void UBinkMediaTexture::PreEditChange( FProperty* PropertyAboutToChange ) 
{
	// this will release the FBinkMediaTextureResource
	Super::PreEditChange(PropertyAboutToChange);

	FlushRenderingCommands();
}


void UBinkMediaTexture::PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent ) 
{
	InitializeTrack();

	// this will recreate the FBinkMediaTextureResource
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

#endif // BINKPLUGIN_UE4_EDITOR

void UBinkMediaTexture::InitializeTrack() 
{
	// assign new media player asset
	if (CurrentMediaPlayer != MediaPlayer) 
	{
		if (CurrentMediaPlayer) 
		{
			CurrentMediaPlayer->OnMediaChanged().RemoveAll(this);
		}

		CurrentMediaPlayer = MediaPlayer;

		if (MediaPlayer) 
		{
			MediaPlayer->OnMediaChanged().AddUObject(this, &UBinkMediaTexture::HandleMediaPlayerMediaChanged);
		}	
	}

	if (MediaPlayer)
	{
		FIntPoint NewDimensions = MediaPlayer->GetDimensions();
		if (NewDimensions != CachedDimensions && NewDimensions != FIntPoint(ForceInit))
		{
			CachedDimensions = NewDimensions;
			UpdateResource();
		}
	}
	else if (CachedDimensions.X != 0 || CachedDimensions.Y != 0)
	{ 
		CachedDimensions = FIntPoint(ForceInit);
		UpdateResource();
	}
}

void UBinkMediaTexture::HandleMediaPlayerMediaChanged() 
{
	InitializeTrack();
}

FTextureResource* UBinkMediaTexture::CreateResource() 
{ 
	//UE_LOGF(LogBink, Log, "Create resource %ls", *this->GetName());
	return new FBinkMediaTextureResource(this, PixelFormat); 
}
