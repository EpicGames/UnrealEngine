// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkMediaTextureResource.h"

#include "DeviceProfiles/DeviceProfileManager.h"
#include "DeviceProfiles/DeviceProfile.h"
#include "BinkMediaPlayer.h"
#include "RenderingThread.h"
#include "Rendering/SlateRenderer.h"

void FBinkMediaTextureResource::InitRHI(FRHICommandListBase& RHICmdListBase) 
{
	FRHICommandListImmediate& RHICmdList = RHICmdListBase.GetAsImmediate();

	int w = Owner->GetSurfaceWidth() > 0 ? Owner->GetSurfaceWidth() : 1;
	int h = Owner->GetSurfaceHeight() > 0 ? Owner->GetSurfaceHeight() : 1;
	// Enforce micro-tile restrictions for render targets.
	w = (w + 7) & -8;
	h = (h + 7) & -8;

	// Create the RHI texture. Only one mip is used and the texture is targetable or resolve.
	ETextureCreateFlags TexCreateFlags = Owner->SRGB ? TexCreate_SRGB : TexCreate_None;
	
	if (bink_force_pixel_format != PF_Unknown) 
	{
		PixelFormat = bink_force_pixel_format;
	}

	// Some platforms don't support srgb 10.10.10.2 formats
	if (PixelFormat == PF_A2B10G10R10) 
	{
		TexCreateFlags = TexCreate_None;
	}

	const TCHAR* DebugName = TEXT("Bink");

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
	FString DebugNameString = TEXT("Bink:");
	DebugNameString += Owner->GetName();
	DebugName = *DebugNameString;
#endif // ARK_EXTRA_RESOURCE_NAMES

	const FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(DebugName)
		.SetExtent(w, h)
		.SetFormat(PixelFormat)
		.SetFlags(TexCreateFlags | ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
		.SetInitialState(ERHIAccess::SRVMask);

	TextureRHI = RenderTargetTextureRHI = RHICmdList.CreateTexture(Desc);

	// Don't bother updating if its not a valid video
	if (Owner->GetSurfaceWidth() && Owner->GetSurfaceHeight()) 
	{
		AddToDeferredUpdateList(false);
	}

	// Create the sampler state RHI resource.
	FSamplerStateInitializerRHI SamplerStateInitializer(
		(ESamplerFilter)UDeviceProfileManager::Get().GetActiveProfile()->GetTextureLODSettings()->GetSamplerFilter(Owner),
		Owner->AddressX == TA_Wrap ? AM_Wrap : (Owner->AddressX == TA_Clamp ? AM_Clamp : AM_Mirror),
		Owner->AddressY == TA_Wrap ? AM_Wrap : (Owner->AddressY == TA_Clamp ? AM_Clamp : AM_Mirror),
		AM_Wrap
	);

	SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);

	RHICmdList.UpdateTextureReference(Owner->TextureReference.TextureReferenceRHI, RenderTargetTextureRHI.GetReference());

	{
		FRHIRenderPassInfo RPInfo(TextureRHI, ERenderTargetActions::Clear_Store);
		RHICmdList.Transition(FRHITransitionInfo(TextureRHI.GetReference(), ERHIAccess::Unknown, ERHIAccess::RTV));
		RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearTexture"));
		RHICmdList.EndRenderPass();
		RHICmdList.SetViewport(0, 0, 0, w, h, 1);
		RHICmdList.Transition(FRHITransitionInfo(TextureRHI.GetReference(), ERHIAccess::RTV, ERHIAccess::SRVGraphics));
	}
}

void FBinkMediaTextureResource::ReleaseRHI() 
{
	FTexture::ReleaseRHI();
	RenderTargetTextureRHI.SafeRelease();
	RemoveFromDeferredUpdateList();
}

void FBinkMediaTextureResource::UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget) 
{
	check(IsInRenderingThread());
	auto Player = Owner->MediaPlayer;
	if (!Player || (!Player->IsPlaying() && !Player->IsPaused()) || !TextureRHI) 
	{
		return;
	}
	FTextureRHIRef tex = TextureRHI->GetTexture2D();
	if (!tex.GetReference()) 
	{
		return;
	}
	uint32 width = tex->GetSizeX();
	uint32 height = tex->GetSizeY();
	bool is_hdr = PixelFormat != PF_B8G8R8A8;
	Player->UpdateTexture(Owner, RHICmdList, tex, tex->GetNativeResource(), width, height, false, Owner->Tonemap, Owner->OutputNits, Owner->Alpha, Owner->DecodeSRGB, is_hdr);
}

void FBinkMediaTextureResource::Clear() 
{
	int w = Owner->GetSurfaceWidth() > 0 ? Owner->GetSurfaceWidth() : 1;
	int h = Owner->GetSurfaceHeight() > 0 ? Owner->GetSurfaceHeight() : 1;

	// Enforce micro-tile restrictions for render targets.
	w = (w + 7) & -8;
	h = (h + 7) & -8;

	FTextureRHIRef ref = RenderTargetTextureRHI;
	FTextureRHIRef ref2 = TextureRHI;
	ENQUEUE_RENDER_COMMAND(BinkMediaPlayer_Draw)([ref,ref2,w,h](FRHICommandListImmediate& RHICmdList) 
	{ 
		FRHIRenderPassInfo RPInfo(ref2, ERenderTargetActions::Clear_Store);
		RHICmdList.Transition(FRHITransitionInfo(ref2.GetReference(), ERHIAccess::Unknown, ERHIAccess::RTV));
		RHICmdList.BeginRenderPass(RPInfo, TEXT("ClearTexture"));
		RHICmdList.EndRenderPass();
		RHICmdList.SetViewport(0, 0, 0, w, h, 1);
		RHICmdList.Transition(FRHITransitionInfo(ref2.GetReference(), ERHIAccess::RTV, ERHIAccess::SRVGraphics));
	});
}

