// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "BinkMediaTexture.h"
#include "TextureResource.h"

struct FBinkMediaTextureResource : FTextureResource, FRenderTarget, FDeferredUpdateResource 
{
	FBinkMediaTextureResource(const UBinkMediaTexture* InOwner, EPixelFormat fmt)
		: Owner(InOwner), PixelFormat(fmt)
	{ 
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override;
	virtual void ReleaseRHI() override;
	virtual FIntPoint GetSizeXY() const override { return Owner->CachedDimensions; }
	virtual void UpdateDeferredResource(FRHICommandListImmediate& RHICmdList, bool bClearRenderTarget = true) override;

	void Clear();

	const UBinkMediaTexture* Owner;
	EPixelFormat PixelFormat;
};
