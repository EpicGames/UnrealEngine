// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkFunctionLibrary.h"

#include "BinkMediaPlayer.h"
#include "BinkMovieStreamer.h"
#include "BinkMediaPlayerPrivate.h"
#include "Engine/GameViewportClient.h"
#include "Engine/Engine.h"
#include "RenderingThread.h"
#include "RHI.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/SlateRenderer.h"
#include "Slate/SlateViewportProvider.h"

extern TSharedPtr<FBinkMovieStreamer, ESPMode::ThreadSafe> MovieStreamer;

// Render-thread draw of all overlay Binks to the backbuffer.
// Each job has already had its bink_textures validated on the game thread.
struct FBinkOverlayRenderJob
{
	FBinkTextures* BinkTextures;
	float ulx, uly, lrx, lry;
	int   tonemap;
	int   out_nits;
};

static void Bink_DrawOverlays_Internal(
	FRHICommandListImmediate& RHICmdList,
	FTextureRHIRef BackBuffer,
	TArray<FBinkOverlayRenderJob> Jobs)
{
	if (!BackBuffer.GetReference() || Jobs.IsEmpty())
	{
		return;
	}

	for (FBinkOverlayRenderJob& job : Jobs)
	{
		if (!job.BinkTextures)
		{
			continue;
		}

		// Determine HDR tonemapping on the render thread where CVar reads are correct
		int32 tonemap, out_nits;
		BinkGetHDRSettings(tonemap, out_nits);

		// Overlay draws load (blend over) existing backbuffer content
		job.BinkTextures->SetRenderTarget(BackBuffer.GetReference(), ERenderTargetLoadAction::ELoad);

		job.BinkTextures->SetHDRSettings(tonemap, 1.0f, out_nits);
		job.BinkTextures->SetAlphaSettings(1.0f, 0);
		job.BinkTextures->SetDrawPosition(job.ulx, job.uly, job.lrx, job.lry);

		job.BinkTextures->Draw();
	}

}

void UBinkFunctionLibrary::Bink_DrawOverlays() 
{
	if (PendingBinkOverlays.IsEmpty())
	{
		return;
	}

	if (!GEngine || !GEngine->GameViewport || !GEngine->GameViewport->Viewport)
	{
		PendingBinkOverlays.Reset();
		return;
	}

	// Collect render jobs — read bink_textures NOW (game thread, after all Ticks).
	// Any Close() that ran during Tick already set bink_textures to null, so we skip those.
	TArray<FBinkOverlayRenderJob> RenderJobs;
	RenderJobs.Reserve(PendingBinkOverlays.Num());

	for (const FBinkOverlayJob& job : PendingBinkOverlays)
	{
		FBinkTextures* tex = job.Player ? job.Player->bink_textures : nullptr;
		if (tex)
		{
			RenderJobs.Add({ tex, job.ulx, job.uly, job.lrx, job.lry, 0, 80 });
		}
	}

	PendingBinkOverlays.Reset();

	if (RenderJobs.IsEmpty())
	{
		return;
	}

	FVector2D ScreenSize;
	GEngine->GameViewport->GetViewportSize(ScreenSize);
	FTextureRHIRef BackBuffer = GEngine->GameViewport->Viewport->GetRenderTargetTexture();

	if (BackBuffer.IsValid())
	{
		ENQUEUE_RENDER_COMMAND(BinkOverlays)(
			[BackBuffer, Jobs = MoveTemp(RenderJobs)](FRHICommandListImmediate& RHICmdList) mutable
		{
			Bink_DrawOverlays_Internal(RHICmdList, BackBuffer, MoveTemp(Jobs));
		});
	}
	else
	{
		TSharedPtr<SWindow> Window = FSlateApplication::Get().GetActiveTopLevelWindow();
		ISlateViewportProvider* ViewportProvider = Window.IsValid() ? FSlateApplication::Get().GetRenderer()->GetViewportProvider(*Window) : nullptr;
		if (ViewportProvider)
		{
			ENQUEUE_RENDER_COMMAND(BinkOverlays)(
				[ViewportProvider, Jobs = MoveTemp(RenderJobs)](FRHICommandListImmediate& RHICmdList) mutable
			{
				FRHITexture* BB = ViewportProvider->GetBackBufferResource();
				if (BB)
				{
					FTextureRHIRef BBRef = BB;
					Bink_DrawOverlays_Internal(RHICmdList, BBRef, MoveTemp(Jobs));
				}
			});
		}
	}
}

FTimespan UBinkFunctionLibrary::BinkLoadingMovie_GetDuration() 
{
	double ms = 0;
	if (MovieStreamer.IsValid() && MovieStreamer.Get()->bnk) 
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(MovieStreamer.Get()->bnk, &info);
		ms = ((double)info.Frames) * ((double)info.FrameRateDiv) * 1000.0 / ((double)info.FrameRate);
	}
	return FTimespan::FromMilliseconds(ms);
}

FTimespan UBinkFunctionLibrary::BinkLoadingMovie_GetTime() 
{
	double ms = 0;
	if (MovieStreamer.IsValid() && MovieStreamer.Get()->bnk) 
	{
		BINKHLINFO info = {};
		BinkHLGetInfo(MovieStreamer.Get()->bnk, &info);
		ms = ((double)info.FrameNum) * ((double)info.FrameRateDiv) * 1000.0 / ((double)info.FrameRate);
	}
	return FTimespan::FromMilliseconds(ms);
}
