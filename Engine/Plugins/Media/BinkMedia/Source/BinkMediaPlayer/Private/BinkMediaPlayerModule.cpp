// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkFunctionLibrary.h"
#include "BinkMediaPlayerPrivate.h"
#include "BinkMoviePlayerSettings.h"
#include "Engine/GameViewportClient.h"
#include "BinkMovieStreamer.h"
#include "Misc/Paths.h"
#include "RenderingThread.h"
#include "HAL/PlatformFileManager.h"
#include "UObject/UObjectIterator.h"
#include "BinkUE.h"
#include "HDRHelper.h"

DEFINE_LOG_CATEGORY(LogBink);

TSharedPtr<FBinkMovieStreamer, ESPMode::ThreadSafe> MovieStreamer;

EPixelFormat bink_force_pixel_format = PF_Unknown;
static bool  bink_initialized        = false;
TArray<FBinkOverlayJob> PendingBinkOverlays;

// ---------------------------------------------------------------------------
// Shared helper implementations
// ---------------------------------------------------------------------------

void BinkGetHDRSettings(int32& OutTonemap, int32& OutNits)
{
	static const auto CVarHDROutputEnabled    = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.EnableHDROutput"));
	static const auto CVarDisplayOutputDevice = IConsoleManager::Get().FindTConsoleVariableDataInt(TEXT("r.HDR.Display.OutputDevice"));

	OutTonemap = 1;
	OutNits    = 80;

	if (GRHISupportsHDROutput && CVarHDROutputEnabled->GetValueOnRenderThread() != 0)
	{
		EDisplayOutputFormat outDev = static_cast<EDisplayOutputFormat>(CVarDisplayOutputDevice->GetValueOnRenderThread());
		float DisplayMaxLuminance  = HDRGetDisplayMaximumLuminance();
		switch (outDev)
		{
		case EDisplayOutputFormat::SDR_sRGB:
		case EDisplayOutputFormat::SDR_Rec709:
		case EDisplayOutputFormat::SDR_ExplicitGammaMapping:
			OutTonemap = 1;
			OutNits    = 80;
			break;
		case EDisplayOutputFormat::HDR_ACES_1000nit_ST2084:
			OutTonemap = 2;
			OutNits    = (int32)DisplayMaxLuminance;
			break;
		case EDisplayOutputFormat::HDR_ACES_1000nit_ScRGB:
			OutTonemap = 1;
			OutNits    = (int32)DisplayMaxLuminance;
			break;
		case EDisplayOutputFormat::HDR_ACES_2000nit_ST2084:
			OutTonemap = 2;
			OutNits    = (int32)DisplayMaxLuminance;
			break;
		case EDisplayOutputFormat::HDR_ACES_2000nit_ScRGB:
			OutTonemap = 1;
			OutNits    = (int32)DisplayMaxLuminance;
			break;
		default:
			OutTonemap = 0;
			OutNits    = 1000;
			break;
		}
	}
}

void BinkCloseOnRenderThread(struct BINK*& InOutBnk, struct FBinkTextures*& InOutTextures)
{
	HBINK OldBnk                = InOutBnk;
	FBinkTextures* OldTextures  = InOutTextures;
	InOutBnk      = nullptr;
	InOutTextures = nullptr;

	if (OldBnk || OldTextures)
	{
		ENQUEUE_RENDER_COMMAND(BinkClose)([OldBnk, OldTextures](FRHICommandListImmediate& RHICmdList)
		{
			if (OldTextures) { OldTextures->Destroy(); }
			if (OldBnk) { BinkClose(OldBnk); }
		});
	}
}

#if PLATFORM_ANDROID
FString BinkGetAndroidAPKPath()
{
	if (JNIEnv* env = FAndroidApplication::GetJavaEnv())
	{
		extern struct android_app* GNativeAndroidApp;
		jclass clazz         = env->GetObjectClass(GNativeAndroidApp->activity->clazz);
		jmethodID methodID   = env->GetMethodID(clazz, "getPackageCodePath", "()Ljava/lang/String;");
		jobject result       = env->CallObjectMethod(GNativeAndroidApp->activity->clazz, methodID);
		jboolean isCopy;
		const char* apkPath  = env->GetStringUTFChars((jstring)result, &isCopy);
		FString Path         = FString(UTF8_TO_TCHAR(apkPath));
		env->ReleaseStringUTFChars((jstring)result, apkPath);
		return Path;
	}
	return FString();
}
#endif

// ---------------------------------------------------------------------------

#if BINKPLUGIN_UE4_EDITOR
#include "Editor.h"
#include "ISettingsModule.h"
#include "Interfaces/IMainFrameModule.h"
#include "BinkMediaPlayer.h"

#define LOCTEXT_NAMESPACE "BinkMediaPlayerModule"
#endif // BINKPLUGIN_UE4_EDITOR

// Expose this function for now so we can control sound system behavior
RADEXPFUNC void RADEXPLINK BinkImmediateCloseSoundSystem( S32 onoff );  // 1 = close once no videos playing (default on desktop), 0=close in BinkFreeGlobals (default on consoles)

bool BinkInitialize() 
{
	if (bink_initialized)
	{
		return true;
	}

	int numSpeakers = GetNumSpeakers();
	if (numSpeakers <= 0) 
	{ 
		numSpeakers = 2; 
	}

	BinkImmediateCloseSoundSystem(0);
	BinkHLInit(4, 44100, (U32)numSpeakers);
	bink_initialized = true;
	return true;
}

FString BinkUE4CookOnTheFlyPath(FString path, const TCHAR *filename) 
{
	FString toPath   = FPaths::ConvertRelativePathToFull(BINKTEMPPATH) + filename;
	FString fromPath = path + filename;
	toPath   = toPath.Replace(TEXT("/./"), TEXT("/"));
	fromPath = fromPath.Replace(TEXT("/./"), TEXT("/"));
	FPlatformFileManager::Get().GetPlatformFile().CopyFile(*toPath, *fromPath);
	return toPath;
}

struct FBinkMediaPlayerModule : IModuleInterface, FTickableGameObject
{
	virtual void StartupModule() override 
	{
		if (IsRunningCommandlet())
		{
			return;
		}

		bPluginInitialized = BinkInitialize();

		if (!bPluginInitialized)
		{
			printf("Bink Error: %s\n", BinkGetError());
		}

		MovieStreamer = MakeShareable(new FBinkMovieStreamer);
		GetMoviePlayer()->RegisterMovieStreamer(MovieStreamer);

#if BINKPLUGIN_UE4_EDITOR
		static ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule && !IsRunningGame())
		{
			SettingsModule->RegisterSettings("Project", "Project", "Bink Movies",
				LOCTEXT("MovieSettingsName", "Bink Movies"),
				LOCTEXT("MovieSettingsDescription", "Bink Movie player settings"),
				GetMutableDefault<UBinkMoviePlayerSettings>()
			);

			FEditorDelegates::BeginPIE.AddRaw(this, &FBinkMediaPlayerModule::HandleEditorTogglePIE);
			FEditorDelegates::EndPIE.AddRaw(this, &FBinkMediaPlayerModule::HandleEditorTogglePIE);
		}
#endif
		GetMutableDefault<UBinkMoviePlayerSettings>()->LoadConfig();
	}

	virtual void ShutdownModule() override 
	{
		if (bPluginInitialized)
		{
			BinkHLShutdown();
			if (overlayHook.IsValid() && GEngine && GEngine->GameViewport)
			{
				GEngine->GameViewport->OnDrawn().Remove(overlayHook);
			}
		}
		MovieStreamer.Reset();
#if BINKPLUGIN_UE4_EDITOR
		FEditorDelegates::BeginPIE.RemoveAll(this);
		FEditorDelegates::EndPIE.RemoveAll(this);
#endif
	}

#if BINKPLUGIN_UE4_EDITOR
	void HandleEditorTogglePIE(bool bIsSimulating)
	{
		for (TObjectIterator<UBinkMediaPlayer> It; It; ++It)
		{
			UBinkMediaPlayer* Player = *It;
			Player->Close();
			if(Player->StartImmediately)
			{
				Player->InitializePlayer();
			}
		}
	}
#endif

	virtual bool IsTickable() const override { return bPluginInitialized; }
	virtual bool IsTickableInEditor() const override { return true; }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FBinkMediaPlayerModule, STATGROUP_Tickables); }

	void DrawBinks() 
	{
		// Decode all playing Binks. This runs on the render thread before
		// UpdateDeferredResource and before any editor texture render commands.
		ENQUEUE_RENDER_COMMAND(BinkProcess)([](FRHICommandListImmediate& RHICmdList)
		{
			BinkHLProcess();
		});

		UBinkFunctionLibrary::Bink_DrawOverlays();
	}

	virtual void Tick(float DeltaTime) override 
	{
		if (GEngine && GEngine->GameViewport) 
		{
			if (overlayHook.IsValid()) 
			{
				GEngine->GameViewport->OnDrawn().Remove(overlayHook);
			}
			overlayHook = GEngine->GameViewport->OnDrawn().AddRaw(this, &FBinkMediaPlayerModule::DrawBinks);
		}
		else 
		{
			DrawBinks();
		}
	}

	FDelegateHandle overlayHook;

#if BINKPLUGIN_UE4_EDITOR
	FDelegateHandle pieBeginHook;
	FDelegateHandle pieEndHook;
#endif
	bool bPluginInitialized = false;
};

IMPLEMENT_MODULE( FBinkMediaPlayerModule, BinkMediaPlayer )

#if BINKPLUGIN_UE4_EDITOR
#undef LOCTEXT_NAMESPACE
#endif
