// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 
#pragma once

#include "Engine/Engine.h"
#include "Rendering/SlateRenderer.h"
#include "AudioMixerDevice.h"
#include "RHIFwd.h"
#include "PixelFormat.h"

#if PLATFORM_ANDROID
#include <android/log.h>
#include <android_native_app_glue.h>
#include "Android/AndroidPlatformFile.h"
#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#endif

#include "egttypes.h"
#include "binktiny.h"
#include "BinkRHI.h"

extern BINKMEDIAPLAYER_API EPixelFormat bink_force_pixel_format;
extern BINKMEDIAPLAYER_API FString BinkUE4CookOnTheFlyPath(FString path, const TCHAR *filename);
extern BINKMEDIAPLAYER_API bool BinkInitialize();

static int GetNumSpeakers() 
{
	if (!GEngine || !GEngine->GetAudioDeviceManager())
	{
		return 2;
	}
	FAudioDevice *dev = FAudioDevice::GetMainAudioDevice().GetAudioDevice();
	if (dev) {
		Audio::FMixerDevice *mix = static_cast<Audio::FMixerDevice*>(dev);
		if (mix) {
			return mix->GetNumDeviceChannels();
		}
	}
	return 2;
}

DECLARE_LOG_CATEGORY_EXTERN(LogBink, Log, All);

// Per-frame overlay draw request, populated during UBinkMediaPlayer::Tick and
// consumed by UBinkFunctionLibrary::Bink_DrawOverlays on the same game thread tick.
struct FBinkOverlayJob
{
	class UBinkMediaPlayer* Player;
	float ulx, uly, lrx, lry;
};

extern TArray<FBinkOverlayJob> PendingBinkOverlays;

// ---------------------------------------------------------------------------
// Shared helpers to eliminate duplicate logic across BinkMediaPlayer,
// BinkFunctionLibrary, and BinkMovieStreamer.
// ---------------------------------------------------------------------------

// Queries HDR CVars on the render thread and returns tonemap mode + output nits.
// Call from the render thread only.
BINKMEDIAPLAYER_API void BinkGetHDRSettings(int32& OutTonemap, int32& OutNits);

// Closes a Bink handle and frees its textures on the render thread.
// Nulls out both pointers. Safe to call from the game thread.
BINKMEDIAPLAYER_API void BinkCloseOnRenderThread(struct BINK*& InOutBnk, struct FBinkTextures*& InOutTextures);

#if PLATFORM_ANDROID
// Retrieves the APK code path via JNI. Returns empty string on failure.
BINKMEDIAPLAYER_API FString BinkGetAndroidAPKPath();
#endif
