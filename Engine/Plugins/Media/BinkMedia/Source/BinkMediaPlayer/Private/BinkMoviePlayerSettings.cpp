// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkMoviePlayerSettings.h"
#include "PixelFormat.h"

UBinkMoviePlayerSettings::UBinkMoviePlayerSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, BinkBufferMode(MP_Bink_Stream)
	, BinkSoundTrack(MP_Bink_Sound_Simple)
	, BinkSoundTrackStart(0)
	, BinkDestinationUpperLeft(0,0)
	, BinkDestinationLowerRight(1,1)
	, BinkPixelFormat(PF_B8G8R8A8)
{
}
