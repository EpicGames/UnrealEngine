// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "BinkMoviePlayerSettings.generated.h"

enum EPixelFormat : uint8;

/**
 * Enumerates available bink buffering modes.
 */
UENUM()
enum EBinkMoviePlayerBinkBufferModes : int
{
	/** Stream the movie off the media during playback (caches about 1 second of video). */
	MP_Bink_Stream UMETA(DisplayName="Stream"),

	/** Loads the whole movie into memory at Open time (will block). */
	MP_Bink_PreloadAll UMETA(DisplayName="Preload All"),

	/** Streams the movie into a memory buffer as big as the movie, so it will be preloaded eventually). */
	MP_Bink_StreamUntilResident UMETA(DisplayName="Stream Until Resident"),

	MP_Bink_MAX,
};

/**
 * Enumerates available used to specify the sounds to open at playback w/ bink movies.
 */
UENUM()
enum EBinkMoviePlayerBinkSoundTrack : int
{
	/** Don't open any sound tracks snd_track_start not used. */
	MP_Bink_Sound_None UMETA(DisplayName="None"),

	/** Based on filename, OR simply mono or stereo sound in track snd_track_start (default speaker spread). */
	MP_Bink_Sound_Simple UMETA(DisplayName="Simple"),

	/** Mono or stereo sound in track 0, language track at snd_track_start. */
	MP_Bink_Sound_LanguageOverride UMETA(DisplayName="Language Override"),

	/** 6 mono tracks in tracks snd_track_start[0..5] */
	MP_Bink_Sound_51 UMETA(DisplayName="5.1 Surround"),

	/** 6 mono tracks in tracks 0..5, center language track at snd_track_start */
	MP_Bink_Sound_51LanguageOverride UMETA(DisplayName="5.1 Surround, Language Override"),

	/** 8 mono tracks in tracks snd_track_start[0..7] */
	MP_Bink_Sound_71 UMETA(DisplayName="7.1 Surround"),

	/** 8 mono tracks in tracks 0..7, center language track at snd_track_start */
	MP_Bink_Sound_71LanguageOverride UMETA(DisplayName="7.1 Surround, Language Override"),

	MP_Bink_Sound_MAX,
};

/**
 * Implements the settings for the Windows target platform.
 */
UCLASS(config=Game, defaultconfig)
class BINKMEDIAPLAYER_API UBinkMoviePlayerSettings
	: public UObject
{
public:

	GENERATED_UCLASS_BODY()

	/** Used to specify the how the video should be buffered. */
	UPROPERTY(globalconfig, EditAnywhere, Category="BinkMovies")
	TEnumAsByte<EBinkMoviePlayerBinkBufferModes> BinkBufferMode;

	/** Used to specify the sounds to open at playback. */
	UPROPERTY(globalconfig, EditAnywhere, Category="BinkMovies")
	TEnumAsByte<EBinkMoviePlayerBinkSoundTrack> BinkSoundTrack;

	/** Used to specify the sounds to open at playback. */
	UPROPERTY(globalconfig, EditAnywhere, Category="BinkMovies")
	int32 BinkSoundTrackStart;

	/** Used to specify the render destination rectangle. */
	UPROPERTY(globalconfig, EditAnywhere, Category="BinkMovies")
	FVector2D BinkDestinationUpperLeft;

	/** Used to specify the render destination rectangle. */
	UPROPERTY(globalconfig, EditAnywhere, Category="BinkMovies")
	FVector2D BinkDestinationLowerRight;

	/** Used to specify the render destination rectangle. */
	UPROPERTY(globalconfig, EditAnywhere, Category="BinkMovies")
	TEnumAsByte<EPixelFormat> BinkPixelFormat;
};

