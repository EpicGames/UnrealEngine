// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA

#include "BinkMovieStreamer.h"

#include "BinkMediaPlayerPrivate.h"
#include "BinkMediaPlayer.h"
#include "BinkMoviePlayerSettings.h"

#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"
#include "RenderingThread.h"
#include "BinkUE.h"

DEFINE_LOG_CATEGORY(LogBinkMoviePlayer);

FBinkMovieStreamer::FBinkMovieStreamer()
	: bnk()
	, bink_textures()
{
	MovieViewport = MakeShareable(new FMovieViewport());
	PlaybackType = MT_Normal;
}

FBinkMovieStreamer::~FBinkMovieStreamer()
{
	CloseMovie();
	Cleanup();

	FlushRenderingCommands();
	TextureFreeList.Empty();
}

bool FBinkMovieStreamer::Init(const TArray<FString>& MoviePaths, TEnumAsByte<EMoviePlaybackType> inPlaybackType)
{
	if (MoviePaths.Num() == 0)
	{
		return false;
	}
	MovieIndex   = -1;
	PlaybackType = inPlaybackType;
	StoredMoviePaths = MoviePaths;
	return OpenNextMovie();
}

FString FBinkMovieStreamer::GetMovieName()
{
	return StoredMoviePaths.IsValidIndex(MovieIndex) ? StoredMoviePaths[MovieIndex] : TEXT("");
}

bool FBinkMovieStreamer::IsLastMovieInPlaylist()
{
	return MovieIndex == StoredMoviePaths.Num() - 1;
}

void FBinkMovieStreamer::ForceCompletion()
{
	CloseMovie();
}

bool FBinkMovieStreamer::Tick(FRHICommandListBase& RHICmdList, float DeltaTime)
{
	if (!bnk)
	{
		CloseMovie();
		if (MovieIndex < StoredMoviePaths.Num() - 1)
		{
			OpenNextMovie();
		}
		else if (PlaybackType != MT_Normal)
		{
			MovieIndex = PlaybackType == MT_LoadingLoop ? StoredMoviePaths.Num() - 2 : -1;
			OpenNextMovie();
		}
		else
		{
			return true;
		}
	}

	BINKHLINFO info = {};
	BinkHLGetInfo(bnk, &info);
	if (info.PlaybackState == 3)
	{
		CloseMovie();
		if (MovieIndex < StoredMoviePaths.Num() - 1)
		{
			OpenNextMovie();
		}
		else if (PlaybackType != MT_Normal)
		{
			MovieIndex = PlaybackType == MT_LoadingLoop ? StoredMoviePaths.Num() - 2 : -1;
			OpenNextMovie();
		}
		else
		{
			return true;
		}
	}
	if (!bnk)
	{
		return false;
	}

	// Decode the next frame for all playing Binks. During loading screens the game thread
	// DrawBinks path may not fire, so the streamer calls BinkHLProcess itself.
	BinkHLProcess();

	FSlateTexture2DRHIRef* CurrentTexture = Texture.Get();
	if (CurrentTexture)
	{
		FVector2D destUpperLeft  = GetDefault<UBinkMoviePlayerSettings>()->BinkDestinationUpperLeft;
		FVector2D destLowerRight = GetDefault<UBinkMoviePlayerSettings>()->BinkDestinationLowerRight;

		if (!CurrentTexture->IsInitialized())
		{
			CurrentTexture->InitResource(RHICmdList);
		}

		FTextureRHIRef tex = CurrentTexture->GetTypedResource();
		uint32 binkw = tex.GetReference()->GetSizeX();
		uint32 binkh = tex.GetReference()->GetSizeY();
		bool is_hdr  = tex.GetReference()->GetFormat() != PF_B8G8R8A8;

		if (bink_textures)
		{
			int32 tonemap, out_nits;
			BinkGetHDRSettings(tonemap, out_nits);

			// The movie streamer runs its own decode+draw loop (no DrawBinks involvement),
			// so draw directly here rather than queuing to PendingBinkTextureDraws.
			bink_textures->SetHDRSettings(tonemap, 1.0f, out_nits);
			bink_textures->SetDrawPosition(
				destUpperLeft.X, destUpperLeft.Y,
				destLowerRight.X, destLowerRight.Y);

			bink_textures->SetRenderTarget(tex.GetReference(), ERenderTargetLoadAction::EClear);

			bink_textures->Draw();
		}

		MovieViewport->SetTexture(Texture);
	}

	return false;
}

void FBinkMovieStreamer::Cleanup()
{
	FlushRenderingCommands();
	for (int32 TextureIndex = 0; TextureIndex < TextureFreeList.Num(); ++TextureIndex)
	{
		BeginReleaseResource(TextureFreeList[TextureIndex].Get());
	}
}

bool FBinkMovieStreamer::OpenNextMovie()
{
	MovieIndex++;
	check(StoredMoviePaths.Num() > 0 && MovieIndex < StoredMoviePaths.Num());

	U32 bufferMode  = (U32)(int)GetDefault<UBinkMoviePlayerSettings>()->BinkBufferMode;
	U32 soundTrack  = (U32)(int)GetDefault<UBinkMoviePlayerSettings>()->BinkSoundTrack;
	U32 soundStart  = (U32)GetDefault<UBinkMoviePlayerSettings>()->BinkSoundTrackStart;

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	FString MoviePathTbl[] =
	{
		StoredMoviePaths[MovieIndex] + TEXT(".bk2")
	};
	for (int i = 0; i < (int)(sizeof(MoviePathTbl) / sizeof(MoviePathTbl[0])) && !bnk; ++i)
	{
		FString FullMoviePath = BINKMOVIEPATH + MoviePathTbl[i];
#if PLATFORM_ANDROID
		if (!IAndroidPlatformFile::GetPlatformPhysical().FileExists(*FullMoviePath)) {
			continue;
		}

		int64 FileOffset     = IAndroidPlatformFile::GetPlatformPhysical().FileStartOffset(*FullMoviePath);
		FString FileRootPath = IAndroidPlatformFile::GetPlatformPhysical().FileRootPath(*FullMoviePath);

		if (IAndroidPlatformFile::GetPlatformPhysical().IsAsset(*FullMoviePath))
		{
			FString APKPath = BinkGetAndroidAPKPath();
			if (!APKPath.IsEmpty())
			{
				bnk = BinkHLOpen(TCHAR_TO_UTF8(*APKPath), soundTrack, soundStart, bufferMode, (U64)FileOffset);
			}
		}
		else
		{
			bnk = BinkHLOpen(TCHAR_TO_UTF8(*FileRootPath), soundTrack, soundStart, bufferMode, (U64)FileOffset);
		}
#else
		if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*FullMoviePath)) 
		{
			FString AbsPath = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*FullMoviePath);
			bnk = BinkHLOpen(TCHAR_TO_UTF8(*AbsPath), soundTrack, soundStart, bufferMode, 0);
		}
		if (!bnk) 
		{
			FString CookPath = *BinkUE4CookOnTheFlyPath(FPaths::ConvertRelativePathToFull(BINKMOVIEPATH), *MoviePathTbl[i]);
			if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*CookPath)) 
			{
				FString AbsPath = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*CookPath);
				bnk = BinkHLOpen(TCHAR_TO_UTF8(*AbsPath), soundTrack, soundStart, bufferMode, 0);
			}
		}
#endif
	}

	if (!bnk)
	{
		return false;
	}

	// Create CPU-side decode buffers immediately. GPU textures created lazily on first Tick.
	bink_textures = FBinkTextures::CreateCPU(bnk, nullptr);

	if (!bink_textures)
	{
		UE_LOGF(LogBinkMoviePlayer, Error, "FBinkMovieStreamer::OpenNextMovie: Failed to create CPU decode buffers.");
		BinkCloseOnRenderThread(bnk, bink_textures);
		return false;
	}

	BINKHLINFO info = {};
	BinkHLGetInfo(bnk, &info);
	FIntPoint VideoDimensions = FIntPoint(info.Width, info.Height);

	if (VideoDimensions.X > 0 && VideoDimensions.Y > 0)
	{
		EPixelFormat pixelFmt = bink_force_pixel_format == PF_Unknown
			? (EPixelFormat)GetDefault<UBinkMoviePlayerSettings>()->BinkPixelFormat
			: bink_force_pixel_format;

		if (TextureFreeList.Num() > 0)
		{
			Texture = TextureFreeList.Pop();

			if (Texture->GetWidth() != VideoDimensions.X || Texture->GetHeight() != VideoDimensions.Y)
			{
				FSlateTexture2DRHIRef* ref = Texture.Get();
				ENQUEUE_RENDER_COMMAND(UpdateMovieTexture)([ref, VideoDimensions](FRHICommandListImmediate& RHICmdList)
				{
					ref->Resize(VideoDimensions.X, VideoDimensions.Y);
				});
			}
		}
		else
		{
			const bool bCreateEmptyTexture = true;
			Texture = MakeShareable(new FSlateTexture2DRHIRef(VideoDimensions.X, VideoDimensions.Y, pixelFmt, NULL, TexCreate_RenderTargetable, bCreateEmptyTexture));
			FSlateTexture2DRHIRef* ref = Texture.Get();

			ENQUEUE_RENDER_COMMAND(InitMovieTexture)([ref](FRHICommandListImmediate& RHICmdList)
			{
				ref->InitResource(RHICmdList);
			});
		}
	}
	return true;
}

void FBinkMovieStreamer::CloseMovie()
{
	if (GetMoviePlayer() != nullptr)
	{
		BroadcastCurrentMovieClipFinished(GetMovieName());
	}

	if (Texture.IsValid())
	{
		TextureFreeList.Add(Texture);
		MovieViewport->SetTexture(NULL);
		Texture.Reset();
	}

	if (bnk)
	{
		BinkCloseOnRenderThread(bnk, bink_textures);
	}
}
