// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA
#pragma once

#include "MoviePlayer.h"

struct BINK;        // forward declare (HBINK = struct BINK*)
struct FBinkTextures; // opaque Bink GPU texture state

class FSlateTexture2DRHIRef;

DECLARE_LOG_CATEGORY_EXTERN(LogBinkMoviePlayer, Log, All);

class FBinkMovieStreamer : public IMovieStreamer 
{
public:
	FBinkMovieStreamer();
	virtual ~FBinkMovieStreamer();

	virtual bool Init(const TArray<FString>& MoviePaths, TEnumAsByte<EMoviePlaybackType> inPlaybackType) override;
	virtual void ForceCompletion() override;
	virtual bool Tick(FRHICommandListBase& RHICmdList, float DeltaTime) override;
	virtual TSharedPtr<class ISlateViewport> GetViewportInterface() override { return MovieViewport; }
	virtual float GetAspectRatio() const override { return (float)MovieViewport->GetSize().X / (float)MovieViewport->GetSize().Y; }
	virtual FString GetMovieName() override;
	virtual bool IsLastMovieInPlaylist() override;
	virtual void Cleanup() override;
	virtual FTextureRHIRef GetTexture() override { return Texture->IsValid() ? Texture->GetRHIRef() : nullptr; }

	FOnCurrentMovieClipFinished OnCurrentMovieClipFinishedDelegate;
	virtual FOnCurrentMovieClipFinished& OnCurrentMovieClipFinished() override { return OnCurrentMovieClipFinishedDelegate; }

	bool OpenNextMovie();
	void CloseMovie();

	TArray<FString> StoredMoviePaths;

	TSharedPtr<FMovieViewport> MovieViewport;
	TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe> Texture;
	TArray<TSharedPtr<FSlateTexture2DRHIRef, ESPMode::ThreadSafe>> TextureFreeList;

	TEnumAsByte<EMoviePlaybackType> PlaybackType;
	int32 MovieIndex;

	struct BINK* bnk;           // HBINK handle
	FBinkTextures* bink_textures; // render-thread GPU texture state (lazily created)
};
