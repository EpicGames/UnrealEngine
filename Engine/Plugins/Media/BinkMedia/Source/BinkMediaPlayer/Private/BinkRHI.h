// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "egttypes.h"
#include "binktiny.h"
#include "RHI.h"
#include "RHICommandList.h"

#define BINKMAXFRAMEBUFFERS 2

// Bink GPU texture state — owns the CPU decode buffers and GPU textures
// for a single playing Bink video. Created by CreateCPU() or Create(),
// destroyed by Destroy(). All draw methods must be called on the render thread.
struct FBinkTextures
{
	// --- Lifecycle -----------------------------------------------------------

	// Full creation (CPU + GPU). Must be called on the render thread.
	BINKMEDIAPLAYER_API static FBinkTextures* Create(HBINK bink, void* user_ptr = nullptr);

	// Creates only the CPU-side decode buffers and registers them with BinkHL.
	// Safe to call from the game thread. GPU textures are created lazily on
	// the render thread the first time Draw() is called.
	BINKMEDIAPLAYER_API static FBinkTextures* CreateCPU(HBINK bink, void* user_ptr = nullptr);

	// Frees all GPU + CPU resources.
	BINKMEDIAPLAYER_API void Destroy();

	// --- Per-frame draw setup (call before Draw) -----------------------------

	BINKMEDIAPLAYER_API void SetRenderTarget(FRHITexture* rt, ERenderTargetLoadAction load_action);
	BINKMEDIAPLAYER_API void SetDrawPosition(float x0, float y0, float x1, float y1);
	BINKMEDIAPLAYER_API void SetDrawCorners(float Ax, float Ay, float Bx, float By, float Cx, float Cy);
	BINKMEDIAPLAYER_API void SetSourceRect(float u0, float v0, float u1, float v1);
	BINKMEDIAPLAYER_API void SetAlphaSettings(float alpha, int32 draw_flags);
	BINKMEDIAPLAYER_API void SetHDRSettings(int32 tonemap, float exposure, int32 out_nits);

	// Draws the current Bink frame to the previously-set render target.
	BINKMEDIAPLAYER_API void Draw();

	// --- Internal state (treat as private) -----------------------------------

	void* user_ptr = nullptr;
	S32 video_width = 0, video_height = 0;
	HBINK bink = nullptr;

	U32 YABufferWidth = 0, YABufferHeight = 0;
	U32 ChBufferWidth = 0, ChBufferHeight = 0;
	S32 NeedAlpha = 0;
	S32 NeedHDR = 0;

	U8* Luma[BINKMAXFRAMEBUFFERS] = {};
	U8* cR[BINKMAXFRAMEBUFFERS] = {};
	U8* cB[BINKMAXFRAMEBUFFERS] = {};
	U8* Alpha[BINKMAXFRAMEBUFFERS] = {};
	U8* HDR[BINKMAXFRAMEBUFFERS] = {};

	S32 bGpuTexturesCreated = 0;

	F32 Ax = 0, Ay = 0, Bx = 0, By = 0, Cx = 0, Cy = 0;
	F32 alpha = 0;
	S32 draw_type = 0;
	S32 draw_flags = 0;
	F32 u0 = 0, v0 = 0, u1 = 0, v1 = 0;

	FTextureRHIRef Ytexture[BINKMAXFRAMEBUFFERS];
	FTextureRHIRef cRtexture[BINKMAXFRAMEBUFFERS];
	FTextureRHIRef cBtexture[BINKMAXFRAMEBUFFERS];
	FTextureRHIRef Atexture[BINKMAXFRAMEBUFFERS];
	FTextureRHIRef Htexture[BINKMAXFRAMEBUFFERS];

	S32 tonemap = 0;
	F32 exposure = 0;
	F32 out_luma = 0;

	FRHITexture* RenderTarget = nullptr;
	ERenderTargetLoadAction RenderTargetLoadAction = ERenderTargetLoadAction::EClear;

private:
	void EnsureGpuTextures();
};
