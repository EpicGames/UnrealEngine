// Copyright Epic Games, Inc. All Rights Reserved.

#include "egttypes.h"
#include "binktiny.h"
#include "BinkRHI.h"
#include "HAL/PlatformFileManager.h"
#include "GenericPlatform/GenericPlatformFile.h"



extern "C"
{

// Returns 1 on success, 0 on failure. Stores the file handle in user_data[0].
S32 RADLINK binkdefopen(UINTa* user_data, const char* fn, U64* optional_out_file_size)
{
	FString Path = UTF8_TO_TCHAR(fn);
	IFileHandle* Handle = FPlatformFileManager::Get().GetPlatformFile().OpenRead(*Path);
	if (!Handle)
	{
		return 0;
	}
	user_data[0] = (UINTa)Handle;
	if (optional_out_file_size)
	{
		*optional_out_file_size = (U64)Handle->Size();
	}
	return 1;
}

// Returns the number of bytes actually read.
U64 RADLINK binkdefread(UINTa* user_data, void* dest, U64 bytes)
{
	IFileHandle* Handle = (IFileHandle*)user_data[0];
	if (!Handle)
	{
		return 0;
	}
	// IFileHandle::Read takes int64; read in chunks if bytes > INT64_MAX (extremely unlikely)
	int64 ToRead = (int64)bytes;
	if (Handle->Read(static_cast<uint8*>(dest), ToRead))
	{
		return (U64)ToRead;
	}
	return 0;
}

// Seeks to an absolute file position.
void RADLINK binkdefseek(UINTa* user_data, U64 pos)
{
	IFileHandle* Handle = (IFileHandle*)user_data[0];
	if (Handle)
	{
		Handle->Seek((int64)pos);
	}
}

// void binkdefclose(UINTa* user_data)
void RADLINK binkdefclose(UINTa* user_data)
{
	IFileHandle* Handle = (IFileHandle*)user_data[0];
	delete Handle;
	user_data[0] = 0;
}

} // extern "C"
