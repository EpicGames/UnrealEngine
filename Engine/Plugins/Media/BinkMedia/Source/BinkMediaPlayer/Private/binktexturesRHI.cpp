// Copyright Epic Games, Inc. All Rights Reserved.

#include "BinkMediaPlayerPrivate.h"
#include "egttypes.h"
#include "binktiny.h"
#include "BinkRHI.h"
#include "BinkShaders.h"
#include "PipelineStateCache.h"
#include "RHIStaticStates.h"
#include "RenderGraphBuilder.h"
#include "RenderingThread.h"

FRDGTextureRef BinkRegisterExternalTexture(FRDGBuilder& GraphBuilder, FRHITexture* Texture, const TCHAR* NameIfUnregistered)
{
	if (FRDGTextureRef FoundTexture = GraphBuilder.FindExternalTexture(Texture))
	{
		return FoundTexture;
	}
	return GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Texture, NameIfUnregistered));
}


// Struct definition is in BinkRHI.h

//-----------------------------------------------------------------------------
BINKMEDIAPLAYER_API FBinkTextures* FBinkTextures::Create(HBINK bink, void* user_ptr)
{
	FBinkTextures* textures = FBinkTextures::CreateCPU(bink, user_ptr);
	if (!textures)
	{
		return nullptr;
	}

	textures->EnsureGpuTextures();
	return textures;
}

// Creates CPU-side decode buffers and registers them with BinkHL.
// Safe to call from the game thread. GPU textures are created lazily
// on the render thread the first time Draw_textures is called.
BINKMEDIAPLAYER_API FBinkTextures* FBinkTextures::CreateCPU(HBINK bink, void* user_ptr)
{
	FBinkTextures* textures = (FBinkTextures*)FMemory::Malloc(sizeof(*textures));
	if (textures == 0)
	{
		return 0;
	}

	BINKHLINFO hlinfo = {};
	BinkHLGetInfo(bink, &hlinfo);

	FMemory::Memset(textures, 0, sizeof(*textures));
	textures->user_ptr = user_ptr;
	textures->bink         = bink;
	textures->video_width  = hlinfo.Width;
	textures->video_height = hlinfo.Height;


	textures->YABufferWidth  = hlinfo.TextureBufferYAWidth;
	textures->YABufferHeight = hlinfo.TextureBufferYAHeight;
	textures->ChBufferWidth  = hlinfo.TextureBufferChWidth;
	textures->ChBufferHeight = hlinfo.TextureBufferChHeight;
	textures->NeedAlpha      = hlinfo.NeedAlpha;
	textures->NeedHDR        = hlinfo.NeedHDR;
	textures->bGpuTexturesCreated = 0;  // GPU textures created lazily on render thread

	const U32 LumaSize   = textures->YABufferWidth * textures->YABufferHeight;
	const U32 ChromaSize = textures->ChBufferWidth  * textures->ChBufferHeight;

	for (int i = 0; i < BINKMAXFRAMEBUFFERS; ++i)
	{
		textures->Luma[i]  = (U8*)FMemory::Malloc(LumaSize);
		textures->cR[i]    = (U8*)FMemory::Malloc(ChromaSize);
		textures->cB[i]    = (U8*)FMemory::Malloc(ChromaSize);
		textures->Alpha[i] = textures->NeedAlpha ? (U8*)FMemory::Malloc(LumaSize)  : nullptr;
		textures->HDR[i]   = textures->NeedHDR   ? (U8*)FMemory::Malloc(LumaSize)  : nullptr;

		if (!textures->Luma[i] || !textures->cR[i] || !textures->cB[i]
			|| (textures->NeedAlpha && !textures->Alpha[i])
			|| (textures->NeedHDR   && !textures->HDR[i]))
		{
			textures->Destroy();
			return nullptr;
		}

		FMemory::Memset(textures->Luma[i],   0,   LumaSize);
		FMemory::Memset(textures->cR[i],    128,  ChromaSize);
		FMemory::Memset(textures->cB[i],    128,  ChromaSize);
		if (textures->Alpha[i]) FMemory::Memset(textures->Alpha[i], 255, LumaSize);
		if (textures->HDR[i])   FMemory::Memset(textures->HDR[i],     0, LumaSize);
	}

	U8* AlphaPtrs[BINKMAXFRAMEBUFFERS] = { textures->Alpha[0], textures->Alpha[1] };
	U8* HDRPtrs[BINKMAXFRAMEBUFFERS]   = { textures->HDR[0],   textures->HDR[1]   };

	BinkHLRegisterTextureBuffers(bink,
		textures->Luma,
		textures->cR,
		textures->cB,
		textures->NeedAlpha ? AlphaPtrs : nullptr,
		textures->NeedHDR   ? HDRPtrs   : nullptr);

	textures->SetDrawCorners(0, 0, 1, 0, 0, 1);
	textures->SetSourceRect(0, 0, 1, 1);
	textures->SetAlphaSettings(1, 0);
	textures->SetHDRSettings(0, 1.0f, 80);

	return textures;
}


//-----------------------------------------------------------------------------
// Creates GPU textures for an already-CPU-initialized FBinkTextures.
// Must be called on the render thread.
void FBinkTextures::EnsureGpuTextures()
{
	if (bGpuTexturesCreated)
	{
		return;
	}

	check(IsInRenderingThread());

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();
	EPixelFormat Format  = PF_R8;
	ETextureCreateFlags TexFlags = TexCreate_NoTiling;

	const FRHITextureCreateDesc YADesc =
		FRHITextureCreateDesc::Create2D(TEXT("BINK"), YABufferWidth, YABufferHeight, Format)
		.SetFlags(TexFlags);
	const FRHITextureCreateDesc ChDesc =
		FRHITextureCreateDesc::Create2D(TEXT("BINK"), ChBufferWidth, ChBufferHeight, Format)
		.SetFlags(TexFlags);

	for (int i = 0; i < BINKMAXFRAMEBUFFERS; ++i)
	{
		Ytexture[i]  = RHICmdList.CreateTexture(YADesc);
		cRtexture[i] = RHICmdList.CreateTexture(ChDesc);
		cBtexture[i] = RHICmdList.CreateTexture(ChDesc);
		if (NeedAlpha) Atexture[i] = RHICmdList.CreateTexture(YADesc);
		if (NeedHDR)   Htexture[i] = RHICmdList.CreateTexture(YADesc);
	}

	bGpuTexturesCreated = 1;
}


//-----------------------------------------------------------------------------
BINKMEDIAPLAYER_API void FBinkTextures::Destroy()
{

	// Clear the flag BEFORE releasing GPU resources so that any concurrent reader
	// seeing this struct (e.g. a stale PendingBinkTextureDraws entry) will know
	// the GPU textures are gone and won't try to draw with them.
	bGpuTexturesCreated = 0;

	for (int i = 0; i < BINKMAXFRAMEBUFFERS; ++i)
	{
		if (Ytexture[i].IsValid())  Ytexture[i].SafeRelease();
		if (cRtexture[i].IsValid()) cRtexture[i].SafeRelease();
		if (cBtexture[i].IsValid()) cBtexture[i].SafeRelease();
		if (Atexture[i].IsValid())  Atexture[i].SafeRelease();
		if (Htexture[i].IsValid())  Htexture[i].SafeRelease();

		if (Luma[i])  { FMemory::Free(Luma[i]);  Luma[i]  = nullptr; }
		if (cR[i])    { FMemory::Free(cR[i]);    cR[i]    = nullptr; }
		if (cB[i])    { FMemory::Free(cB[i]);    cB[i]    = nullptr; }
		if (Alpha[i]) { FMemory::Free(Alpha[i]); Alpha[i] = nullptr; }
		if (HDR[i])   { FMemory::Free(HDR[i]);   HDR[i]   = nullptr; }
	}

	FMemory::Free(this);
}

//-----------------------------------------------------------------------------
BINKMEDIAPLAYER_API void FBinkTextures::Draw()
{

	if (RenderTarget == nullptr)
	{
		return;
	}

	// Create GPU textures on first use (safe: always called from render thread)
	EnsureGpuTextures();

	int hasAPlane = NeedAlpha;
	int hasHPlane = NeedHDR;

	BINKHLINFO Info;
	BinkHLGetInfo(bink, &Info);
	if (Info.TextureDrawFrame == 0)
	{
		return;
	}
	int Idx = (int)Info.TextureDrawIndex;
	if (Idx < 0 || Idx >= BINKMAXFRAMEBUFFERS)
	{
		UE_LOGF(LogBink, Error, "FBinkTextures::Draw: TextureDrawIndex %d out of range [0, %d)", Idx, BINKMAXFRAMEBUFFERS);
		return;
	}

	FRDGBuilder BinkGraphBuilder(FRHICommandListImmediate::Get());
	FBinkParameters* consts = BinkGraphBuilder.AllocParameters<FBinkParameters>();

	FRDGTexture* BinkRDGTexture = BinkRegisterExternalTexture(BinkGraphBuilder, RenderTarget, TEXT("Bink_RT"));
	consts->RenderTargets[0] = FRenderTargetBinding(BinkRDGTexture, RenderTargetLoadAction, 0);
	consts->tex0 = BinkRegisterExternalTexture(BinkGraphBuilder, Ytexture[Idx],  TEXT("BinkY"));
	consts->tex1 = BinkRegisterExternalTexture(BinkGraphBuilder, cRtexture[Idx], TEXT("BinkCr"));
	consts->tex2 = BinkRegisterExternalTexture(BinkGraphBuilder, cBtexture[Idx], TEXT("BinkCb"));
	consts->tex3 = BinkRegisterExternalTexture(BinkGraphBuilder, hasAPlane ? Atexture[Idx] : Ytexture[Idx], TEXT("BinkA"));
	consts->tex4 = BinkRegisterExternalTexture(BinkGraphBuilder, hasHPlane ? Htexture[Idx] : Ytexture[Idx], TEXT("BinkH"));
	consts->samp0 = TStaticSamplerState<SF_Bilinear>::GetRHI();
	consts->samp1 = TStaticSamplerState<SF_Bilinear>::GetRHI();
	consts->samp2 = TStaticSamplerState<SF_Bilinear>::GetRHI();
	consts->samp3 = TStaticSamplerState<SF_Bilinear>::GetRHI();
	consts->samp4 = TStaticSamplerState<SF_Bilinear>::GetRHI();

	if (draw_type == 3)
	{
		unsigned BinkColorA = 0xffffffff;
		unsigned BinkColorB = 0xffc0c0c0;
		consts->consta.X = ((F32)(S32)((BinkColorA >>  0) & 255)) * (1.0f / 255.0f);
		consts->consta.Y = ((F32)(S32)((BinkColorA >>  8) & 255)) * (1.0f / 255.0f);
		consts->consta.Z = ((F32)(S32)((BinkColorA >> 16) & 255)) * (1.0f / 255.0f);
		consts->consta.W = ((F32)(S32)((BinkColorA >> 24) & 255)) * (1.0f / 255.0f);
		consts->crc.X = ((F32)(S32)((BinkColorB >>  0) & 255)) * (1.0f / 255.0f);
		consts->crc.Y = ((F32)(S32)((BinkColorB >>  8) & 255)) * (1.0f / 255.0f);
		consts->crc.Z = ((F32)(S32)((BinkColorB >> 16) & 255)) * (1.0f / 255.0f);
		consts->crc.W = ((F32)(S32)((BinkColorB >> 24) & 255)) * (1.0f / 255.0f);
		consts->cbc.X = ((F32)video_width)  / 8.0f;
		consts->cbc.Y = ((F32)video_height) / 8.0f;
		consts->cbc.Z = 0.f;
		consts->cbc.W = 0.f;
	}
	else
	{
		consts->consta.Z = consts->consta.Y = consts->consta.X = draw_type == 1 ? alpha : 1.0f;
		consts->consta.W = alpha;

		if (hasHPlane)
		{
			// HDR stuff
			consts->crc.X = Info.cs0;
			consts->crc.Y = exposure;
			consts->crc.Z = out_luma;
			consts->crc.W = 0;
			consts->cbc.X = Info.cs1;
			consts->cbc.Y = Info.cs2;
			consts->cbc.Z = Info.cs3;
			consts->cbc.W = Info.cs4;
		}
		else
		{
			// set the constants for the type of ycrcb we have
			consts->crc.X = Info.cs0;
			consts->crc.Y = Info.cs1;
			consts->crc.Z = Info.cs2;
			consts->crc.W = Info.cs3;
			consts->cbc.X = Info.cs4;
			consts->cbc.Y = Info.cs5;
			consts->cbc.Z = Info.cs6;
			consts->cbc.W = Info.cs7;
			consts->adj.X = Info.cs8;
			consts->adj.Y = Info.cs9;
			consts->adj.Z = Info.cs10;
			consts->adj.W = Info.cs11;
			consts->yscale.X = Info.cs12;
			consts->yscale.Y = Info.cs13;
			consts->yscale.Z = Info.cs14;
			consts->yscale.W = Info.cs15;
		}
	}

	consts->xy_xform0.X = (Bx - Ax) * 2.0f;
	consts->xy_xform0.Y = (Cx - Ax) * 2.0f;
	consts->xy_xform0.Z = (By - Ay) * -2.0f;
	consts->xy_xform0.W = (Cy - Ay) * -2.0f;
	consts->xy_xform1.X = Ax * 2.0f - 1.0f;
	consts->xy_xform1.Y = 1.0f - Ay * 2.0f;
	consts->xy_xform1.Z = 0.0f;
	consts->xy_xform1.W = 0.0f;

	{
		F32 luma_u   = (F32)video_width        / (F32)YABufferWidth;
		F32 luma_v   = (F32)video_height       / (F32)YABufferHeight;
		F32 chroma_u = (F32)(video_width  / 2) / (F32)ChBufferWidth;
		F32 chroma_v = (F32)(video_height / 2) / (F32)ChBufferHeight;

		consts->uv_xform0.X = (u1 - u0) * luma_u;
		consts->uv_xform0.Y = 0.0f;
		consts->uv_xform0.Z = (u1 - u0) * chroma_u;
		consts->uv_xform0.W = 0.0f;

		consts->uv_xform1.X = 0.0f;
		consts->uv_xform1.Y = (v1 - v0) * luma_v;
		consts->uv_xform1.Z = 0.0f;
		consts->uv_xform1.W = (v1 - v0) * chroma_v;

		consts->uv_xform2.X = u0 * luma_u;
		consts->uv_xform2.Y = v0 * luma_v;
		consts->uv_xform2.Z = u0 * chroma_u;
		consts->uv_xform2.W = v0 * chroma_v;
	}

	FBinkDrawVS::FParameters* vert_params = BinkGraphBuilder.AllocParameters<FBinkDrawVS::FParameters>();
	vert_params->BinkParameters = *consts;

	FBinkDrawICtCpPS::FParameters* ictcp_params = BinkGraphBuilder.AllocParameters<FBinkDrawICtCpPS::FParameters>();
	ictcp_params->BinkParameters = *consts;

	FBinkDrawYCbCrPS::FParameters* ycbcr_params = BinkGraphBuilder.AllocParameters<FBinkDrawYCbCrPS::FParameters>();
	ycbcr_params->BinkParameters = *consts;

	auto* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FBinkDrawVS> DrawVS(ShaderMap);

	FBinkDrawICtCpPS::FPermutationDomain ictcp_pv;
	ictcp_pv.Set<FBinkDrawICtCpPS::FALPHA>(!!hasAPlane);
	ictcp_pv.Set<FBinkDrawICtCpPS::FTONEMAP>(tonemap == 1);
	ictcp_pv.Set<FBinkDrawICtCpPS::FST2084>(tonemap == 2);
	TShaderMapRef<FBinkDrawICtCpPS> DrawICtCpPS(ShaderMap, ictcp_pv);

	FBinkDrawYCbCrPS::FPermutationDomain ycbcr_pv;
	ycbcr_pv.Set<FBinkDrawYCbCrPS::FALPHA>(!!hasAPlane);
	ycbcr_pv.Set<FBinkDrawYCbCrPS::FSRGB>(!!(draw_flags & 0x80000000));
	TShaderMapRef<FBinkDrawYCbCrPS> DrawYCbCrPS(ShaderMap, ycbcr_pv);

	FRDGEventName EventName(TEXT("Bink"));
	BinkGraphBuilder.AddPass(
		MoveTemp(EventName),
		consts,
		ERDGPassFlags::Raster | ERDGPassFlags::SkipRenderPass,
		[&](FRHICommandListImmediate& RHICmdList)
		{
			FUpdateTextureRegion2D region_YAH(0, 0, 0, 0, YABufferWidth, YABufferHeight);
			FUpdateTextureRegion2D region_Ch(0, 0, 0, 0,  ChBufferWidth,  ChBufferHeight);

			RHICmdList.UpdateTexture2D(Ytexture[Idx],  0, region_YAH, YABufferWidth, (uint8*)Luma[Idx]);
			RHICmdList.UpdateTexture2D(cRtexture[Idx], 0, region_Ch,  ChBufferWidth,  (uint8*)cR[Idx]);
			RHICmdList.UpdateTexture2D(cBtexture[Idx], 0, region_Ch,  ChBufferWidth,  (uint8*)cB[Idx]);
			if (hasAPlane && Alpha[Idx])
			{
				RHICmdList.UpdateTexture2D(Atexture[Idx], 0, region_YAH, YABufferWidth, (uint8*)Alpha[Idx]);
			}
			if (hasHPlane && HDR[Idx])
			{
				RHICmdList.UpdateTexture2D(Htexture[Idx], 0, region_YAH, YABufferWidth, (uint8*)HDR[Idx]);
			}

			RHICmdList.Transition(FRHITransitionInfo(consts->RenderTargets[0].GetTexture()->GetRHI()->GetTexture2D(), ERHIAccess::Unknown, ERHIAccess::RTV));
			FRHIRenderPassInfo RPInfo(consts->RenderTargets[0].GetTexture()->GetRHI()->GetTexture2D(), MakeRenderTargetActions(RenderTargetLoadAction, ERenderTargetStoreAction::EStore));
			RHICmdList.BeginRenderPass(RPInfo, TEXT("BinkPass"));

			FGraphicsPipelineStateInitializer GraphicsPSOInit;
			RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

			if ((!hasAPlane && alpha >= 0.999f) || draw_type == 2)
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
			}
			else if (draw_type == 1)
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_One, BF_InverseSourceAlpha, BO_Add, BF_One, BF_Zero>::GetRHI();
			}
			else
			{
				GraphicsPSOInit.BlendState = TStaticBlendState<CW_RGBA, BO_Add, BF_SourceAlpha, BF_InverseSourceAlpha, BO_Add, BF_One, BF_Zero>::GetRHI();
			}

			GraphicsPSOInit.RasterizerState   = TStaticRasterizerState<FM_Solid, CM_None, ERasterizerDepthClipMode::DepthClip, false>::GetRHI();
			GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

			FVertexDeclarationElementList Elements;
			GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
			GraphicsPSOInit.BoundShaderState.VertexShaderRHI      = DrawVS.GetVertexShader();
			GraphicsPSOInit.PrimitiveType = PT_TriangleStrip;

			if (hasHPlane)
			{
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = DrawICtCpPS.GetPixelShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParameters(RHICmdList, DrawICtCpPS, DrawICtCpPS.GetPixelShader(), *ictcp_params);
			}
			else
			{
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = DrawYCbCrPS.GetPixelShader();
				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
				SetShaderParameters(RHICmdList, DrawYCbCrPS, DrawYCbCrPS.GetPixelShader(), *ycbcr_params);
			}

			SetShaderParameters(RHICmdList, DrawVS, DrawVS.GetVertexShader(), *vert_params);
			RHICmdList.DrawPrimitive(0, 2, 1);
			RHICmdList.EndRenderPass();
		});

	BinkGraphBuilder.Execute();
}

BINKMEDIAPLAYER_API void FBinkTextures::SetDrawPosition(float x0, float y0, float x1, float y1)
{
	Ax = x0; Ay = y0;
	Bx = x1; By = y0;
	Cx = x0; Cy = y1;
}

BINKMEDIAPLAYER_API void FBinkTextures::SetDrawCorners(float InAx, float InAy, float InBx, float InBy, float InCx, float InCy)
{
	Ax = InAx; Ay = InAy;
	Bx = InBx; By = InBy;
	Cx = InCx; Cy = InCy;
}

BINKMEDIAPLAYER_API void FBinkTextures::SetSourceRect(float InU0, float InV0, float InU1, float InV1)
{
	u0 = InU0; v0 = InV0;
	u1 = InU1; v1 = InV1;
}

BINKMEDIAPLAYER_API void FBinkTextures::SetAlphaSettings(float InAlpha, int32 InDrawFlags)
{
	alpha      = InAlpha;
	draw_type  = InDrawFlags & 0x0FFFFFFF;
	draw_flags = InDrawFlags & 0xF0000000;
}

BINKMEDIAPLAYER_API void FBinkTextures::SetHDRSettings(int32 InTonemap, float InExposure, int32 InOutNits)
{
	tonemap  = InTonemap;
	exposure = InExposure;
	out_luma = ((F32)InOutNits) / 80.f;
}

BINKMEDIAPLAYER_API void FBinkTextures::SetRenderTarget(FRHITexture* rt, ERenderTargetLoadAction load_action)
{
	RenderTarget = rt;
	RenderTargetLoadAction = load_action;
}
