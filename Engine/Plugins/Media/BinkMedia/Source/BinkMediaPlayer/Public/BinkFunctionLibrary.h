// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "BinkFunctionLibrary.generated.h"

UCLASS()
class BINKMEDIAPLAYER_API UBinkFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:
	
	UFUNCTION(BlueprintCallable, Category = "Bink")
	static void Bink_DrawOverlays();

	UFUNCTION(BlueprintCallable, Category = "Bink")
	static FTimespan BinkLoadingMovie_GetDuration();

	UFUNCTION(BlueprintCallable, Category = "Bink")
	static FTimespan BinkLoadingMovie_GetTime();
}; 

