// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "Tickable.h"
#include "Delegates/DelegateCombinations.h"
#include "RHI.h"
#include "Slate/SlateTextures.h"
#include "BinkMediaPlayer.generated.h"

struct BINK;        // forward declare for HBINK (typedef struct BINK* HBINK in binktiny.h)
struct FBinkTextures; // opaque Bink GPU texture state

extern BINKMEDIAPLAYER_API EPixelFormat bink_force_pixel_format;
extern BINKMEDIAPLAYER_API FString BinkUE4CookOnTheFlyPath(FString path, const TCHAR *filename);

/**
 * Enumerates available bink buffering modes.
 */
UENUM()
enum EBinkMediaPlayerBinkBufferModes : int
{
	/** Stream the movie off the media during playback (caches about 1 second of video). */
	BMASM_Bink_Stream UMETA(DisplayName="Stream"),

	/** Loads the whole movie into memory at Open time (will block). */
	BMASM_Bink_PreloadAll UMETA(DisplayName="Preload All"),

	/** Streams the movie into a memory buffer as big as the movie, so it will be preloaded eventually). */
	BMASM_Bink_StreamUntilResident UMETA(DisplayName="Stream Until Resident"),

	BMASM_Bink_MAX,
};

/**
 * Enumerates available used to specify the sounds to open at playback w/ bink movies.
 */
UENUM()
enum EBinkMediaPlayerBinkSoundTrack : int
{
	/** Don't open any sound tracks snd_track_start not used. */
	BMASM_Bink_Sound_None UMETA(DisplayName="None"),

	/** Based on filename, OR simply mono or stereo sound in track snd_track_start (default speaker spread). */
	BMASM_Bink_Sound_Simple UMETA(DisplayName="Simple"),

	/** Mono or stereo sound in track 0, language track at snd_track_start. */
	BMASM_Bink_Sound_LanguageOverride UMETA(DisplayName="Language Override"),

	/** 6 mono tracks in tracks snd_track_start[0..5] */
	BMASM_Bink_Sound_51 UMETA(DisplayName="5.1 Surround"),

	/** 6 mono tracks in tracks 0..5, center language track at snd_track_start */
	BMASM_Bink_Sound_51LanguageOverride UMETA(DisplayName="5.1 Surround, Language Override"),

	/** 8 mono tracks in tracks snd_track_start[0..7] */
	BMASM_Bink_Sound_71 UMETA(DisplayName="7.1 Surround"),

	/** 8 mono tracks in tracks 0..7, center language track at snd_track_start */
	BMASM_Bink_Sound_71LanguageOverride UMETA(DisplayName="7.1 Surround, Language Override"),

	BMASM_Bink_Sound_MAX,
};

/**
 * Enumerates available bink buffering modes.
 */
UENUM()
enum EBinkMediaPlayerBinkDrawStyle : int
{
	/** Renders movie to the destination texture (default UE4 functionality) */
	BMASM_Bink_DS_RenderToTexture UMETA(DisplayName="Render to Texture"),

	/** Renders movie in an overlay (UE4 bypass) and corrects for movie aspect ratio. */
	BMASM_Bink_DS_OverlayFillScreenWithAspectRatio UMETA(DisplayName="Overlay Fill Screen with Aspect Ratio"),

	/** Renders movie in an overlay (UE4 bypass), displaying the original movie size and not stretching to fill frame */
	BMASM_Bink_DS_OverlayOriginalMovieSize UMETA(DisplayName="Overlay Fill Original Movie Size"),

	/** Renders movie in an overlay (UE4 bypass), filling the entire destination rectangle */
	BMASM_Bink_DS_OverlayFillScreen UMETA(DisplayName="Overlay Fill Screen"),

	/** Renders movie in an overlay (UE4 bypass), rendering to the specific rectangle specified (same as FillScreen) */
	BMASM_Bink_DS_OverlaySpecificDestinationRectangle UMETA(DisplayName="Overlay Specific Destination Rectangle"),

	BMASM_Bink_DS_MAX,
};

/** Multicast delegate that is invoked when a media player's media has been closed. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnBinkMediaPlayerMediaClosed);

/** Multicast delegate that is invoked when a media player's media has been opened. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnBinkMediaPlayerMediaOpened, FString, OpenedUrl);

/** Multicast delegate that is invoked when a media player's media has finished playing. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnBinkMediaPlayerMediaReachedEnd);

/** Multicast delegate that is invoked when a media event occurred in the player. */
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnBinkMediaPlayerMediaEvent);

/**
 * Implements a media player asset that can play movies and other media.
 *
 * This class is represents a media URL along with a corresponding media player
 * for exposing media playback functionality to the Engine and to Blueprints.
 */
UCLASS(BlueprintType, hidecategories=(Object))
class BINKMEDIAPLAYER_API UBinkMediaPlayer : public UObject, public FTickableGameObject 
{
	GENERATED_UCLASS_BODY()
public:
	/**
	 * Checks whether media playback can be paused right now.
	 *
	 * Playback can be paused if the media supports pausing and if it is currently playing.
	 *
	 * @return true if pausing playback can be paused, false otherwise.
	 * @see CanPlay, Pause
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool CanPause() const;

	/**
	 * Checks whether media playback can be started right now.
	 *
	 * @return true if playback can be started, false otherwise.
	 * @see CanPause, Play
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool CanPlay() const;

	/**
	 * Gets the media's duration.
	 *
	 * @return A time span representing the duration.
	 * @see GetTime, Seek
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	FTimespan GetDuration() const;

	/**
	 * Gets the media's current playback rate.
	 *
	 * @return The playback rate.
	 * @see SetRate, SupportsRate
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	float GetRate() const;

	/**
	 * Gets the media's current playback time.
	 *
	 * @return Playback time.
	 * @see GetDuration, Seek
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	FTimespan GetTime() const;

	/**
	 * Gets the URL of the currently loaded media, if any.
	 *
	 * @return Media URL, or empty string if no media was loaded.
	 * @see OpenUrl
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	const FString& GetUrl() const;

	/**
	 * Checks whether playback is looping.
	 *
	 * @return true if looping, false otherwise.
	 * @see SetLooping
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool IsLooping() const;

	/**
	 * Checks whether playback is currently paused.
	 *
	 * @return true if playback is paused, false otherwise.
	 * @see CanPause, IsPlaying, IsStopped, Pause
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool IsPaused() const;

	/**
	 * Checks whether playback has started.
	 *
	 * @return true if playback has started, false otherwise.
	 * @see CanPlay, IsPaused, IsStopped, Play
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool IsPlaying() const;

	/**
	 * Checks whether playback has stopped.
	 *
	 * @return true if playback has stopped, false otherwise.
	 * @see IsPaused, IsPlaying, Stop
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool IsStopped() const;

	/**
	 * Checks whether playback is "GoTo'ing".
	 *
	 * @return true if playback is gotoing, false otherwise.
	 * @see IsPaused, IsPlaying, Stop
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool IsGotoing() const;

	/**
	 * Opens the specified media URL.
	 *
	 * @param NewUrl The URL to open.
	 * @return true on success, false otherwise.
	 * @see GetUrl
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool OpenUrl( const FString& NewUrl );

	/**
	 * Closes the specified media URL.
	 *
	 * @param NewUrl The URL to open.
	 * @return true on success, false otherwise.
	 * @see GetUrl
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	void CloseUrl();

	/**
	 * Pauses media playback.
	 *
	 * This is the same as setting the playback rate to 0.0.
	 *
	 * @return true if playback is being paused, false otherwise.
	 * @see CanPause, Play, Rewind, Seek, SetRate
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool Pause();

	/**
	 * Starts media playback.
	 *
	 * This is the same as setting the playback rate to 1.0.
	 *
	 * @return true if playback is starting, false otherwise.
	 * @see CanPlay, Pause, Rewind, Seek, SetRate
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool Play();

	/**
	 * Rewinds the media to the beginning.
	 *
	 * This is the same as seeking to zero time.
	 *
	 * @return true if rewinding, false otherwise.
	 * @see GetTime, Pause, Play, Seek
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool Rewind();

	/**
	 * Seeks to the specified playback time.
	 *
	 * @param InTime The playback time to set.
	 * @return true on success, false otherwise.
	 * @see GetTime, Rewind
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool Seek( const FTimespan& InTime );

	/**
	 * Enables or disables playback looping.
	 *
	 * @param Looping Whether playback should be looped.
	 * @return true on success, false otherwise.
	 * @see IsLooping
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool SetLooping( bool InLooping );

	/**
	 * Changes the media's playback rate.
	 *
	 * @param Rate The playback rate to set.
	 * @return true on success, false otherwise.
	 * @see GetRate, SupportsRate
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool SetRate( float Rate );

	/**
	 * Changes the media's volume
	 *
	 * @param Rate The playback volume to set. 0 to 1
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	void SetVolume( float Rate );

	/**
	 * Stops playback and unloads the video from memory. If you want to start the video again you'll need to call InitializePlayer.
	 * MediaEvent will broadcast EMediaEvent::MediaClosed.
	 * @return true on success, false otherwise.
	 * @see InitializePlayer, IsStopped, Play
	 */
	UFUNCTION(BlueprintCallable, Category = "Media|BinkMediaPlayer")
	void Stop() { Close(); }

	/**
	 * Checks whether the specified playback rate is supported.
	 *
	 * @param Rate The playback rate to check.
	 * @param Unthinned Whether no frames should be dropped at the given rate.
	 * @see SupportsScrubbing, SupportsSeeking
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool SupportsRate( float Rate, bool Unthinned ) const;

	/**
	 * Checks whether the currently loaded media supports scrubbing.
	 *
	 * @return true if scrubbing is supported, false otherwise.
	 * @see SupportsRate, SupportsSeeking
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool SupportsScrubbing() const;

	/**
	 * Checks whether the currently loaded media can jump to a certain position.
	 *
	 * @return true if seeking is supported, false otherwise.
	 * @see SupportsRate, SupportsScrubbing
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool SupportsSeeking() const;

	/**
	 * Checks whether this player has been initialized with a media source.
	 *
	 * @return true if a media source is associated with this player.
	 * @see OpenUrl
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	bool IsInitialized() const { return IsReady(); }

	/**
	 * Draws this bink to the specified texture
	 */
	UFUNCTION(BlueprintCallable, Category="Media|BinkMediaPlayer")
	void Draw(UTexture *texture, bool tonemap=false, int out_nits=10000, float alpha=1, bool srgb_decode=false, bool hdr=false);

	/** Gets an event delegate that is invoked when media has been opened or closed. */
	DECLARE_EVENT(UBinkMediaPlayer, FOnMediaChanged)
	FOnMediaChanged& OnMediaChanged() { return MediaChangedEvent; }

	/** Holds a delegate that is invoked when a media source has been closed. */
	UPROPERTY(BlueprintAssignable, Category="Media|BinkMediaPlayer")
	FOnBinkMediaPlayerMediaClosed OnMediaClosed;

	/** Holds a delegate that is invoked when a media source has been opened. */
	UPROPERTY(BlueprintAssignable, Category="Media|BinkMediaPlayer")
	FOnBinkMediaPlayerMediaOpened OnMediaOpened;

	/** Holds a delegate that is invoked when a media source has been opened. */
	UPROPERTY(BlueprintAssignable, Category="Media|BinkMediaPlayer")
	FOnBinkMediaPlayerMediaReachedEnd OnMediaReachedEnd;

	/** A delegate that is invoked when media playback has been suspended. */
	UPROPERTY(BlueprintAssignable, Category = "Media|BinkMediaPlayer", meta = (HideInDetailPanel))
	FOnBinkMediaPlayerMediaEvent OnPlaybackSuspended;

	// UMediaPlayerBase overrides not already part of the build provided by RAD
	//virtual TSharedRef<FMediaPlayerFacade, ESPMode::ThreadSafe> GetPlayerFacade() const override;
	//virtual bool IsPreparing() const override { return !IsReady(); }
	//virtual bool OpenSource(UMediaSource* InMediaSource) override;

	// UObject overrides.

	virtual void BeginDestroy() override;
	virtual FString GetDesc() override;
	virtual void PostLoad() override;

#if BINKPLUGIN_UE4_EDITOR
	virtual void PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent ) override;
#endif

	/** Initializes the media player. */
	void InitializePlayer();

	/** Whether playback should loop when it reaches the end. */
	UPROPERTY(EditAnywhere, Category=Playback)
	uint32 Looping:1;

	/** Whether playback start immediately, or wait for blueprint to start it. */
	UPROPERTY(EditAnywhere, Category=Playback)
	uint32 StartImmediately:1;

	/** To reduce memory use, don't immediately open the bink until it is first played. */
	UPROPERTY(EditAnywhere, Category=Playback)
	uint32 DelayedOpen:1;

	/** Used to specify the sounds to open at playback. */
	UPROPERTY(EditAnywhere, Category=Playback)
	FVector2D BinkDestinationUpperLeft;

	/** Used to specify the sounds to open at playback. */
	UPROPERTY(EditAnywhere, Category=Playback)
	FVector2D BinkDestinationLowerRight;

	/** The path or URL to the media file to be played. */
	UPROPERTY(EditAnywhere, Category=Source)
	FString URL;

	/** Used to specify the how the video should be buffered. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Source)
	TEnumAsByte<EBinkMediaPlayerBinkBufferModes> BinkBufferMode;

	/** Used to specify the sounds to open at playback. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Source)
	TEnumAsByte<EBinkMediaPlayerBinkSoundTrack> BinkSoundTrack;

	/** Used to specify the sounds to open at playback. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Source)
	int32 BinkSoundTrackStart;

	/** Used to specify how the movie is drawn. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Source)
	TEnumAsByte<EBinkMediaPlayerBinkDrawStyle> BinkDrawStyle;

	/** Used to specify the render order. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Source)
	int32 BinkLayerDepth;

	/** Holds the bink buffer mode of the currently loaded media source. */
	TEnumAsByte<EBinkMediaPlayerBinkBufferModes> CurrentBinkBufferMode;

	/** Holds the bink sound track of the currently loaded media source. */
	TEnumAsByte<EBinkMediaPlayerBinkSoundTrack> CurrentBinkSoundTrack;

	/** Holds the bink sound track start of the currently loaded media source. */
	int32 CurrentBinkSoundTrackStart;

	/** Callback for when the media player has closed a media source. */
	void HandleMediaPlayerMediaClosed();

	/** Callback for when the media player has opened a new media source. */
	void HandleMediaPlayerMediaOpened( FString OpenedUrl );

	// Note - not serialized. 
	FString CurrentUrl;
	EBinkMediaPlayerBinkDrawStyle CurrentDrawStyle;
	int32 CurrentLayerDepth;
	int32 CurrentHasSubtitles;

	/** Holds a delegate that is executed when media has been opened or closed. */
	FOnMediaChanged MediaChangedEvent;

	// --------

	// FTickableGameObject Interface
    
	void Tick(float DeltaTime);
	bool IsTickableInEditor() const { return true; }
	TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(UBinkMediaPlayer, STATGROUP_Tickables); }
	bool IsTickable() const { return true; }

	bool Open(const FString& Url);
	void Close();
	bool IsReady() const { return bnk != NULL; }
	void UpdateTexture(const UTexture* tex, FRHICommandListImmediate &RHICmdList, FTextureRHIRef ref, void *nativePtr, int width, int height, bool isEditor, bool tonemap, int output_nits, float alpha, bool srgb_decode, bool is_hdr);


	FIntPoint GetDimensions() const;
	float GetFrameRate() const;

	struct BINK *bnk;              // HBINK handle (struct BINK* as defined in binktiny.h)
	FBinkTextures *bink_textures; // render-thread GPU texture state (lazily created)
	bool paused;
	bool reached_end;
	FDelegateHandle overlayHook;

};
