// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "Runtime/Launch/Resources/Version.h"
#include "Engine/Texture.h"
#include "BinkMediaTexture.generated.h"

class UBinkMediaPlayer;
class FBinkMediaPlayer;
enum EPixelFormat : uint8;
enum TextureAddress : int;

/**
 * Implements a texture asset for rendering video tracks from UBinkMediaPlayer assets.
 * 
 * note: derives directly from UTexture, not from UTexture2D or UTexture2DDynamic
 *    maybe should have been UTexture2DDynamic?
 */
UCLASS(hidecategories=(Compression, LevelOfDetail, Object))
class BINKMEDIAPLAYER_API UBinkMediaTexture : public UTexture 
{
	GENERATED_UCLASS_BODY()

	/** The addressing mode to use for the X axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MediaTexture, AssetRegistrySearchable)
	TEnumAsByte<TextureAddress> AddressX;

	/** The addressing mode to use for the Y axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=MediaTexture, AssetRegistrySearchable)
	TEnumAsByte<TextureAddress> AddressY;

	~UBinkMediaTexture();

	// UTexture overrides.

	virtual FTextureResource* CreateResource() override;
	virtual EMaterialValueType GetMaterialType() const override { return MCT_Texture2D; }
	virtual float GetSurfaceWidth() const override { return CachedDimensions.X; }
	virtual float GetSurfaceHeight() const override { return CachedDimensions.Y; }
	virtual float GetSurfaceDepth() const override { return 0; }
	virtual uint32 GetSurfaceArraySize() const override { return 0; }
	virtual ETextureClass GetTextureClass() const { return ETextureClass::Other2DNoSource; }

	// UObject overrides.

	virtual void BeginDestroy() override;
	virtual void FinishDestroy() override;
	virtual FString GetDesc() override;
	virtual bool IsReadyForFinishDestroy() override { return (Super::IsReadyForFinishDestroy() && ReleasePlayerFence && ReleasePlayerFence->IsFenceComplete()); }
	virtual void PostLoad() override;

#if BINKPLUGIN_UE4_EDITOR
	virtual void PreEditChange( FProperty* PropertyAboutToChange ) override;
	virtual void PostEditChangeProperty( FPropertyChangedEvent& PropertyChangedEvent ) override;
#endif

	/**
	 * Sets the media player asset to be used for this texture.
	 *
	 * @param InMediaPlayer The asset to set.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaTexture")
	void SetMediaPlayer( UBinkMediaPlayer* InMediaPlayer );

	/**
	 * Clears this texture to transparent-black.
	 */
	UFUNCTION(BlueprintCallable, Category="Media|MediaTexture")
	void Clear();


	/** The MediaPlayer asset to stream video from. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MediaPlayer)
	TObjectPtr<UBinkMediaPlayer> MediaPlayer;

	/** The Pixel Format for the texture. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MediaPlayer)
	TEnumAsByte<EPixelFormat> PixelFormat;

	/** Whether to enable tonemaping for the video. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MediaPlayer)
	bool Tonemap;

	/** When tonemapping, set this for to desired output nits capability for the current display. Typically 80 for SDR, or 2000 for HDR. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = MediaPlayer)
	float OutputNits;

	/** alpha_value is just a constant blend value for entire video frame. 1 (default) opaque, 0 fully transparent. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = MediaPlayer)
	float Alpha;

	/** Enable decoding of sRGB to Linear inside Bink when drawing to this texture. You could use this when rendering to a texture format which doesn't support sRGB for example. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=MediaPlayer)
	bool DecodeSRGB;

	void InitializeTrack();
	void HandleMediaPlayerMediaChanged();
	FIntPoint CachedDimensions;
	UBinkMediaPlayer *CurrentMediaPlayer;
	FRenderCommandFence* ReleasePlayerFence;
};
