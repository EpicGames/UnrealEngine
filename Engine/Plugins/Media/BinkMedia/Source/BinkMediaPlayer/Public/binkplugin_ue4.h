// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#ifndef BINKPLUGIN_UE4_H
#define BINKPLUGIN_UE4_H

// you can define these variables at the project compiler define level
//   if you want the Bink data or binaries to live somewhere else

#ifndef BINKCONTENTPATH
#define BINKCONTENTPATH FPaths::ProjectContentDir()
#endif

#ifndef BINKMOVIEPATH
#define BINKMOVIEPATH BINKCONTENTPATH + TEXT("Movies/")
#endif

#ifndef BINKTEMPPATH
#define BINKTEMPPATH FPaths::ProjectSavedDir() + TEXT("Bink/")
#endif

#endif // BINKPLUGIN_UE4_H

