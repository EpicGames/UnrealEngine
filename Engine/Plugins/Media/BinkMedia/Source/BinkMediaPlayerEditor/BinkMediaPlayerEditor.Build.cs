// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class BinkMediaPlayerEditor : ModuleRules 
{
	public BinkMediaPlayerEditor(ReadOnlyTargetRules Target) : base(Target) 
	{
		DynamicallyLoadedModuleNames.AddRange( new string[] { "AssetTools", "MainFrame", "WorkspaceMenuStructure" } );

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ContentBrowser",
				"Core",
				"CoreUObject",			
				"Engine",
				"InputCore",
				"PropertyEditor",
				"RenderCore",
				"RHI",
				"Slate",
				"SlateCore",
				"TextureEditor",
				"UnrealEd",
				"DesktopPlatform",
				"BinkMediaPlayer",
			}
		);

		PublicDefinitions.Add("BUILDING_FOR_UNREAL_ONLY=1");
		PublicDefinitions.Add("__RADNOEXPORTS__=1");
		PublicDefinitions.Add("__RADINSTATICLIB__=1");
		PublicDependencyModuleNames.Add("BinkMediaPlayer");

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"UnrealEd",
				"WorkspaceMenuStructure",
			}
		);

        if (Target.bBuildEditor) 
        {
            PublicDefinitions.Add("BINKPLUGIN_UE4_EDITOR=1");
        } 
        else 
        {
            PublicDefinitions.Add("BINKPLUGIN_UE4_EDITOR=0");
        }

        if (Target.Platform == UnrealTargetPlatform.Mac) 
        {
            PublicDependencyModuleNames.Add("MetalRHI");
            AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
        }
    }
}
