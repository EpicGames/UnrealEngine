// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkMediaPlayerCustomization.h"

#include "BinkMediaPlayer.h"
#include "DesktopPlatformModule.h"
#include "DetailWidgetRow.h"
#include "Framework/Application/SlateApplication.h"
#include "IDesktopPlatform.h"
#include "Widgets/Images/SImage.h"
#include "EditorDirectories.h"
#include "DetailLayoutBuilder.h"
#include "Misc/Paths.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/SWindow.h"
#include "BinkUE.h"

#define LOCTEXT_NAMESPACE "FBinkMediaPlayerCustomization"

DECLARE_DELEGATE_OneParam(FBinkOnPathPicked, const FString& /* InOutPath */);
class SBinkFilePathPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SBinkFilePathPicker ){}
	SLATE_EVENT(FBinkOnPathPicked, OnPathPicked)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs) {
		OnPathPicked = InArgs._OnPathPicked;

		ChildSlot
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "HoverHintOnly" )
				.ToolTipText( LOCTEXT( "FileButtonToolTipText", "Choose a file from this computer") )
				.OnClicked( FOnClicked::CreateSP(this, &SBinkFilePathPicker::OnPickFile) )
				.ContentPadding( 2.0f )
				.ForegroundColor( FSlateColor::UseForeground() )
				.IsFocusable( false )
				[
					SNew( SImage )
					.Image( FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis") )
					.ColorAndOpacity( FSlateColor::UseForeground() )
				]
			]
		];
	}

private:
	/** Delegate handling the picker button being clicked */
	FReply OnPickFile() 
	{
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if ( DesktopPlatform ) 
		{
			const FString DefaultPath = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN);

			TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
			void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid()) ? ParentWindow->GetNativeWindow()->GetOSWindowHandle() : nullptr;
			FString Filter = TEXT("Bink 2 files (*.bk2)|*.bk2");

			TArray<FString> OutFiles;
		
			if (DesktopPlatform->OpenFileDialog(ParentWindowHandle, LOCTEXT("PropertyEditorTitle", "File picker...").ToString(), DefaultPath, TEXT(""), Filter, EFileDialogFlags::None, OutFiles)) 
			{
				FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(OutFiles[0]));

				if(OnPathPicked.IsBound()) 
				{
					OnPathPicked.Execute(OutFiles[0]);
				}
			}
		}

		return FReply::Handled();
	}

private:

	/** Delegate fired when a path is picked */
	FBinkOnPathPicked OnPathPicked;
};

void FBinkMediaPlayerCustomization::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) 
{
	DetailBuilder.GetObjectsBeingCustomized(CustomizedMediaPlayers);

	// customize 'Source' category
	IDetailCategoryBuilder& SourceCategory = DetailBuilder.EditCategory("Source");
	{
		// URL
		UrlProperty = DetailBuilder.GetProperty("URL");
		{
			IDetailPropertyRow& UrlRow = SourceCategory.AddProperty(UrlProperty);

			UrlRow.DisplayName(LOCTEXT("URLLabel", "URL"));
			UrlRow.CustomWidget()
				.NameContent()
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
								.Font(IDetailLayoutBuilder::GetDetailFont())
								.Text(LOCTEXT("FileOrUrlPropertyName", "File or URL"))
								.ToolTipText(UrlProperty->GetToolTipText())
						]

					+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(SImage)
								.Image(FCoreStyle::Get().GetBrush("Icons.Warning"))
								.ToolTipText(LOCTEXT("InvalidUrlPathWarning", "The current URL points to a file that does not exist or is not located inside the /Content/Movies/ directory."))
								.Visibility(this, &FBinkMediaPlayerCustomization::HandleUrlWarningIconVisibility)
						]
				]
				.ValueContent()
				.MaxDesiredWidth(0.0f)
				.MinDesiredWidth(125.0f)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SEditableText)
								.Font(IDetailLayoutBuilder::GetDetailFontBold())
								.Text_Lambda([=, this]()
									{
										FText UrlValueText;
										UrlProperty->GetValueAsDisplayText(UrlValueText);
										return UrlValueText;
									})
								.OnTextCommitted_Lambda([this](const FText& Val, ETextCommit::Type TextCommitType)
									{
										if (TextCommitType != ETextCommit::OnEnter)
											return;

										FString NewPathStr = Val.ToString();
										HandleUrlPickerPathPicked(NewPathStr);
									})
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0f)
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Center)
						.Padding(4.0f, 0.0f, 4.0f, 0.0f)
						[
							SNew(SBinkFilePathPicker)
								.OnPathPicked(this, &FBinkMediaPlayerCustomization::HandleUrlPickerPathPicked)
						]
				];
		}
	}

	// customize 'Information' category
	IDetailCategoryBuilder& InformationCategory = DetailBuilder.EditCategory("Information", FText::GetEmpty(), ECategoryPriority::Uncommon);
	{
		// duration
		InformationCategory.AddCustomRow(LOCTEXT("Duration", "Duration"))
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("Duration", "Duration"))
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			]
			.ValueContent()
			.MaxDesiredWidth(0.0f)
			[
				SNew(STextBlock)
					.Text(this, &FBinkMediaPlayerCustomization::HandleDurationTextBlockText)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
					.ToolTipText(LOCTEXT("DurationToolTip", "The total duration of this media, i.e. how long it plays"))
			];
	}
}


/* FBinkMediaPlayerCustomization callbacks
 *****************************************************************************/

FText FBinkMediaPlayerCustomization::HandleDurationTextBlockText() const 
{
	TOptional<FTimespan> Duration;

	// ensure that all assets have the same value
	for (auto MediaPlayerObject : CustomizedMediaPlayers) 
	{
		if (MediaPlayerObject.IsValid()) 
		{
			FTimespan OtherDuration = Cast<UBinkMediaPlayer>(MediaPlayerObject.Get())->GetDuration();

			if (!Duration.IsSet()) 
			{
				Duration = OtherDuration;
			}
			else if (OtherDuration != Duration.GetValue()) 
			{
				return FText::GetEmpty();
			}
		}
	}

	if (!Duration.IsSet()) 
	{
		return FText::GetEmpty();
	}

	return FText::AsTimespan(Duration.GetValue());
}

void FBinkMediaPlayerCustomization::HandleUrlPickerPathPicked( const FString& PickedPath ) 
{
	if (PickedPath.IsEmpty() || PickedPath.StartsWith(TEXT("./")) || PickedPath.Contains(TEXT("://"))) 
	{
		UrlProperty->SetValue(PickedPath);
	}
	else 
	{	
		FString FullUrl = FPaths::ConvertRelativePathToFull(PickedPath);
		const FString FullGameContentDir = FPaths::ConvertRelativePathToFull( BINKCONTENTPATH );

		if (FullUrl.StartsWith(FullGameContentDir)) 
		{
			FPaths::MakePathRelativeTo(FullUrl, *FullGameContentDir);
			FullUrl = FString(TEXT("./")) + FullUrl;
		}

		UrlProperty->SetValue(FullUrl);
	}
}


EVisibility FBinkMediaPlayerCustomization::HandleUrlWarningIconVisibility() const 
{
	FString Url;

	if ((UrlProperty->GetValue(Url) != FPropertyAccess::Success) || Url.IsEmpty() || Url.Contains(TEXT("://"))) 
	{
		return EVisibility::Hidden;
	}

	const FString FullMoviesPath = FPaths::ConvertRelativePathToFull( BINKMOVIEPATH );
	const FString FullUrl = FPaths::ConvertRelativePathToFull(FPaths::IsRelative(Url) ? BINKCONTENTPATH / Url : Url);

	if (FullUrl.StartsWith(FullMoviesPath)) 
	{
		if (FPaths::FileExists(FullUrl)) 
		{
			return EVisibility::Hidden;
		}

		// file doesn't exist
		return EVisibility::Visible;
	}

	// file not inside Movies folder
	return EVisibility::Visible;
}


#undef LOCTEXT_NAMESPACE
