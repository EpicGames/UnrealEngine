// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "IDetailCustomization.h"
#include "PropertyHandle.h"

struct EVisibility;

enum class EMediaPlaybackDirections;

struct FBinkMediaPlayerCustomization : public IDetailCustomization 
{
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

	static TSharedRef<IDetailCustomization> MakeInstance() 
	{ 
		return MakeShareable(new FBinkMediaPlayerCustomization()); 
	}

	FText HandleDurationTextBlockText() const;
	FString HandleUrlPickerFilePath() const 
	{ 
		FString Url; 
		UrlProperty->GetValue(Url); 
		return Url; 
	}
	FString HandleUrlPickerFileTypeFilter() const 
	{ 
		return TEXT("Bink 2 files (*.bk2)|*.bk2"); 
	}
	void HandleUrlPickerPathPicked( const FString& PickedPath );
	EVisibility HandleUrlWarningIconVisibility() const;

	TArray<TWeakObjectPtr<UObject>> CustomizedMediaPlayers;
	TSharedPtr<IPropertyHandle> UrlProperty;
};
