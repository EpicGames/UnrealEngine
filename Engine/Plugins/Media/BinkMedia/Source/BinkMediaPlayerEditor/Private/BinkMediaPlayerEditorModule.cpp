// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 


#include "AssetTypeActions_Base.h"
#include "BinkMediaPlayerCustomization.h"
#include "Models/BinkMediaPlayerEditorCommands.h"
#include "ContentBrowserModule.h"
#include "BinkMediaPlayerEditorToolkit.h"
#include "Factories/MaterialFactoryNew.h"
#include "BinkMediaTexture.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Factories/BinkMediaTextureFactoryNew.h"
#include "IContentBrowserSingleton.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/ITextureEditorModule.h"
#include "Materials/Material.h"
#include "Misc/PackageName.h"
#include "Styling/SlateStyleRegistry.h"
#include "Misc/Paths.h"
#include "ThumbnailRendering/TextureThumbnailRenderer.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"

#define LOCTEXT_NAMESPACE "FBinkMediaPlayerEditorModule"

struct FBinkMediaPlayerEditorStyle final : FSlateStyleSet
{
	FBinkMediaPlayerEditorStyle() : FSlateStyleSet("BinkMediaPlayerEditorStyle")
	{
		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Media/MediaPlayerEditor/Content"));

		// toolbar icons
		FVector2D Icon20x20(20, 20);
		FVector2D Icon40x40(40, 40);
		Set("BinkMediaPlayerEditor.PauseMedia", new IMAGE_BRUSH("icon_pause_40x", Icon40x40));
		Set("BinkMediaPlayerEditor.PauseMedia.Small", new IMAGE_BRUSH("icon_pause_40x", Icon20x20));
		Set("BinkMediaPlayerEditor.PlayMedia", new IMAGE_BRUSH("icon_play_40x", Icon40x40));
		Set("BinkMediaPlayerEditor.PlayMedia.Small", new IMAGE_BRUSH("icon_play_40x", Icon20x20));
		Set("BinkMediaPlayerEditor.RewindMedia", new IMAGE_BRUSH("icon_rewind_40x", Icon40x40));
		Set("BinkMediaPlayerEditor.RewindMedia.Small", new IMAGE_BRUSH("icon_rewind_40x", Icon20x20));
		Set("BinkMediaPlayerEditor.StopMedia", new IMAGE_BRUSH("icon_stop_40x", Icon40x40));
		Set("BinkMediaPlayerEditor.StopMedia.Small", new IMAGE_BRUSH("icon_stop_40x", Icon20x20));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	~FBinkMediaPlayerEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};

struct FBinkMediaTextureActions : FAssetTypeActions_Base 
{
	virtual void GetActions( const TArray<UObject*>& InObjects, FMenuBuilder& MenuBuilder ) override 
	{
		FAssetTypeActions_Base::GetActions(InObjects, MenuBuilder);

		auto Textures = GetTypedWeakObjectPtrs<UTexture>(InObjects);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("MediaTexture_CreateMaterial", "Create Material"),
			LOCTEXT("MediaTexture_CreateMaterialTooltip", "Creates a new material using this texture."),
			FSlateIcon( FAppStyle::GetAppStyleSetName(), "ClassIcon.Material" ),
			FUIAction(
				FExecuteAction::CreateSP(this, &FBinkMediaTextureActions::ExecuteCreateMaterial, Textures),
				FCanExecuteAction()
			)
		);
	}
	
	virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>() ) override 
	{
		EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

		for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt) 
		{
			auto Texture = Cast<UTexture>(*ObjIt);
			if (Texture) 
			{
				ITextureEditorModule* TextureEditorModule = &FModuleManager::LoadModuleChecked<ITextureEditorModule>("TextureEditor");
				TextureEditorModule->CreateTextureEditor(Mode, EditWithinLevelEditor, Texture);
			}
		}
	}

	void ExecuteCreateMaterial( TArray<TWeakObjectPtr<UTexture>> Objects ) 
	{
		IContentBrowserSingleton& ContentBrowserSingleton = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
		const FString DefaultSuffix = TEXT("_Mat");

		if (Objects.Num() == 1) 
		{
			auto Object = Objects[0].Get();
			if (Object) 
			{
				// Determine an appropriate name
				FString Name;
				FString PackagePath;
				CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);

				// Create the factory used to generate the asset
				UMaterialFactoryNew* Factory = NewObject<UMaterialFactoryNew>();
				Factory->InitialTexture = Object;

				ContentBrowserSingleton.CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), UMaterial::StaticClass(), Factory);
			}
		} 
		else 
		{
			TArray<UObject*> ObjectsToSync;

			for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt) 
			{
				auto Object = (*ObjIt).Get();
				if (Object != nullptr) 
				{
					FString Name;
					FString PackageName;
					CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

					// Create the factory used to generate the asset
					UMaterialFactoryNew* Factory = NewObject<UMaterialFactoryNew>();
					Factory->InitialTexture = Object;

					FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
					UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), UMaterial::StaticClass(), Factory);
					if (NewAsset) 
					{
						ObjectsToSync.Add(NewAsset);
					}
				}
			}
			if (ObjectsToSync.Num() > 0) 
			{
				ContentBrowserSingleton.SyncBrowserToAssets(ObjectsToSync);
			}
		}
	}

	virtual bool CanFilter() override 
	{ 
		return true; 
	}
	virtual uint32 GetCategories() override 
	{ 
		return EAssetTypeCategories::Materials | EAssetTypeCategories::Textures; 
	}
	virtual FText GetName() const override 
	{ 
		return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_MediaTexture", "Media Texture"); 
	}
	virtual UClass* GetSupportedClass() const override 
	{ 
		return UBinkMediaTexture::StaticClass(); 
	}
	virtual FColor GetTypeColor() const override 
	{ 
		return FColor::Red; 
	}
};

struct FBinkMediaPlayerActions : FAssetTypeActions_Base 
{
	FBinkMediaPlayerActions( const TSharedRef<ISlateStyle>& InStyle ) : Style(InStyle) 
	{ 
	}

	virtual void GetActions( const TArray<UObject*>& InObjects, FMenuBuilder& MenuBuilder ) override 
	{
		FAssetTypeActions_Base::GetActions(InObjects, MenuBuilder);

		auto MediaPlayers = GetTypedWeakObjectPtrs<UBinkMediaPlayer>(InObjects);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("MediaPlayer_PlayMovie", "Play Movie"),
			LOCTEXT("MediaPlayer_PlayMovieToolTip", "Starts playback of the media."),
			FSlateIcon( FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Play" ),
			FUIAction(
				FExecuteAction::CreateSP(this, &FBinkMediaPlayerActions::HandlePlayMovieActionExecute, MediaPlayers),
				FCanExecuteAction()
			)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("MediaPlayer_PauseMovie", "Pause Movie"),
			LOCTEXT("MediaPlayer_PauseMovieToolTip", "Pauses playback of the media."),
			FSlateIcon( FAppStyle::GetAppStyleSetName(), "MediaAsset.AssetActions.Pause" ),
			FUIAction(
				FExecuteAction::CreateSP(this, &FBinkMediaPlayerActions::HandlePauseMovieActionExecute, MediaPlayers),
				FCanExecuteAction()
			)
		);

		MenuBuilder.AddMenuSeparator();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("MediaPlayer_CreateMediaTexture", "Create Media Texture"),
			LOCTEXT("MediaPlayer_CreateMediaTextureTooltip", "Creates a new MediaTexture using this BinkMediaPlayer asset."),
			FSlateIcon( FAppStyle::GetAppStyleSetName(), "ClassIcon.MediaTexture" ),
			FUIAction(
				FExecuteAction::CreateSP(this, &FBinkMediaPlayerActions::ExecuteCreateMediaTexture, MediaPlayers),
				FCanExecuteAction()
			)
		);

		FAssetTypeActions_Base::GetActions(InObjects, MenuBuilder);
	}

	virtual void OpenAssetEditor( const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor = TSharedPtr<IToolkitHost>() ) override 
	{
		EToolkitMode::Type Mode = EditWithinLevelEditor.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone;

		for (auto ObjIt = InObjects.CreateConstIterator(); ObjIt; ++ObjIt) 
		{
			auto MediaPlayer = Cast<UBinkMediaPlayer>(*ObjIt);
			if (MediaPlayer) 
			{
				TSharedRef<FBinkMediaPlayerEditorToolkit> EditorToolkit = MakeShareable(new FBinkMediaPlayerEditorToolkit(Style));
				EditorToolkit->Initialize(MediaPlayer, Mode, EditWithinLevelEditor);
			}
		}
	}

	void ExecuteCreateMediaTexture( TArray<TWeakObjectPtr<UBinkMediaPlayer>> Objects ) 
	{
		IContentBrowserSingleton& ContentBrowserSingleton = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
		const FString DefaultSuffix = TEXT("_Tex");

		if (Objects.Num() == 1) 
		{
			auto Object = Objects[0].Get();
			if (Object) 
			{
				// Determine an appropriate name
				FString Name;
				FString PackagePath;
				CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackagePath, Name);

				// Create the factory used to generate the asset
				UBinkMediaTextureFactoryNew* Factory = NewObject<UBinkMediaTextureFactoryNew>();
				Factory->InitialMediaPlayer = Object;

				ContentBrowserSingleton.CreateNewAsset(Name, FPackageName::GetLongPackagePath(PackagePath), UBinkMediaTexture::StaticClass(), Factory);
			}
		} 
		else 
		{
			TArray<UObject*> ObjectsToSync;
			for (auto ObjIt = Objects.CreateConstIterator(); ObjIt; ++ObjIt) 
			{
				auto Object = (*ObjIt).Get();
				if (Object) 
				{
					FString Name;
					FString PackageName;
					CreateUniqueAssetName(Object->GetOutermost()->GetName(), DefaultSuffix, PackageName, Name);

					// Create the factory used to generate the asset
					UBinkMediaTextureFactoryNew* Factory = NewObject<UBinkMediaTextureFactoryNew>();
					Factory->InitialMediaPlayer = Object;

					FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");
					UObject* NewAsset = AssetToolsModule.Get().CreateAsset(Name, FPackageName::GetLongPackagePath(PackageName), UBinkMediaTexture::StaticClass(), Factory);
					if (NewAsset) 
					{
						ObjectsToSync.Add(NewAsset);
					}
				}
			}

			if (ObjectsToSync.Num() > 0) 
			{
				ContentBrowserSingleton.SyncBrowserToAssets(ObjectsToSync);
			}
		}
	}

	void HandlePauseMovieActionExecute( TArray<TWeakObjectPtr<UBinkMediaPlayer>> Objects ) 
	{
		for (auto MediaPlayer : Objects) 
		{
			if (MediaPlayer.IsValid()) 
			{
				MediaPlayer->Pause();
			}
		}
	}

	void HandlePlayMovieActionExecute( TArray<TWeakObjectPtr<UBinkMediaPlayer>> Objects ) 
	{
		for (auto MediaPlayer : Objects) 
		{
			if (MediaPlayer.IsValid()) 
			{
				MediaPlayer->Play();
			}
		}
	}

	virtual bool CanFilter() override { return true; }
	virtual uint32 GetCategories() override { return EAssetTypeCategories::Misc; }
	virtual FText GetName() const override { return FText::FromString(TEXT("Bink Media Player")); }
	virtual UClass* GetSupportedClass() const override { return UBinkMediaPlayer::StaticClass(); }
	virtual FColor GetTypeColor() const override { return FColor::Red; }

	TSharedRef<ISlateStyle> Style;
};

struct FBinkMediaPlayerEditorModule : public IHasMenuExtensibility, public IHasToolBarExtensibility, public IModuleInterface 
{

	virtual void StartupModule() override 
	{
		Style = MakeShareable(new FBinkMediaPlayerEditorStyle());

		FBinkMediaPlayerEditorCommands::Register();

		IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();

		auto reg = [&](TSharedRef<IAssetTypeActions> Action) 
		{
			AssetTools.RegisterAssetTypeActions(Action);
			RegisteredAssetTypeActions.Add(Action);
		};
		reg(MakeShareable(new FBinkMediaPlayerActions(Style.ToSharedRef())));
		reg(MakeShareable(new FBinkMediaTextureActions));

		MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
		ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

		UThumbnailManager::Get().RegisterCustomRenderer(UBinkMediaTexture::StaticClass(), UTextureThumbnailRenderer::StaticClass());

		static FName PropertyEditor("PropertyEditor");
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(PropertyEditor);
		PropertyModule.RegisterCustomClassLayout("BinkMediaPlayer", FOnGetDetailCustomizationInstance::CreateStatic(&FBinkMediaPlayerCustomization::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();
	}

	virtual void ShutdownModule() override 
	{
		FAssetToolsModule* AssetToolsModule = FModuleManager::GetModulePtr<FAssetToolsModule>("AssetTools");
		if (AssetToolsModule) 
		{
			IAssetTools& AssetTools = AssetToolsModule->Get();
			for (auto Action : RegisteredAssetTypeActions) 
			{
				AssetTools.UnregisterAssetTypeActions(Action);
			}
		}

		MenuExtensibilityManager.Reset();
		ToolBarExtensibilityManager.Reset();

		if (UObjectInitialized()) 
		{
			UThumbnailManager::Get().UnregisterCustomRenderer(UBinkMediaTexture::StaticClass());
		}

		static FName PropertyEditor("PropertyEditor");
		FModuleManager& ModuleManager = FModuleManager::Get();
		if (ModuleManager.IsModuleLoaded(PropertyEditor)) 
		{
			FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
			PropertyModule.UnregisterCustomClassLayout("BinkMediaPlayer");
			PropertyModule.NotifyCustomizationModuleChanged();
		}
	}

	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

	TSharedPtr<FExtensibilityManager> MenuExtensibilityManager;
	TArray<TSharedRef<IAssetTypeActions>> RegisteredAssetTypeActions;
	TSharedPtr<ISlateStyle> Style;
	TSharedPtr<FExtensibilityManager> ToolBarExtensibilityManager;
};


IMPLEMENT_MODULE(FBinkMediaPlayerEditorModule, BinkMediaPlayerEditor);


#undef LOCTEXT_NAMESPACE
