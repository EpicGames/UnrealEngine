// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkMediaPlayerEditorToolkit.h"

#include "Models/BinkMediaPlayerEditorCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Docking/SDockTab.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SBinkWidgets.h"

#define LOCTEXT_NAMESPACE "FBinkMediaPlayerEditorToolkit"

DEFINE_LOG_CATEGORY_STATIC(LogMediaPlayerEditor, Log, All);

static const FName DetailsTabId("Details");
static const FName MediaPlayerEditorAppIdentifier("BinkMediaPlayerEditorApp");
static const FName ViewerTabId("Viewer");

void FBinkMediaPlayerEditorToolkit::Initialize( UBinkMediaPlayer* InMediaPlayer, const EToolkitMode::Type InMode, const TSharedPtr<class IToolkitHost>& InToolkitHost ) 
{
	MediaPlayer = InMediaPlayer;

	MediaPlayer->InitializePlayer();
	MediaPlayer->SetFlags(RF_Transactional);
	GEditor->RegisterForUndo(this);

	FEditorDelegates::EndPIE.AddRaw(this, &FBinkMediaPlayerEditorToolkit::HandleEditorEndPIE);

	const FBinkMediaPlayerEditorCommands& Commands = FBinkMediaPlayerEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.PauseMedia,
		FExecuteAction::CreateSP(this, &FBinkMediaPlayerEditorToolkit::HandlePauseMediaActionExecute),
		FCanExecuteAction::CreateSP(this, &FBinkMediaPlayerEditorToolkit::HandlePauseMediaActionCanExecute));

	ToolkitCommands->MapAction(
		Commands.PlayMedia,
		FExecuteAction::CreateSP(this, &FBinkMediaPlayerEditorToolkit::HandlePlayMediaActionExecute),
		FCanExecuteAction::CreateSP(this, &FBinkMediaPlayerEditorToolkit::HandlePlayMediaActionCanExecute));

	ToolkitCommands->MapAction(
		Commands.RewindMedia,
		FExecuteAction::CreateSP(this, &FBinkMediaPlayerEditorToolkit::HandleRewindMediaActionExecute),
		FCanExecuteAction::CreateSP(this, &FBinkMediaPlayerEditorToolkit::HandleRewindMediaActionCanExecute));

	const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("BinkMediaPlayerEditor_Layout_v2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					FTabManager::NewStack()
						->AddTab(ViewerTabId, ETabState::OpenedTab)
						->SetHideTabWell(true)
						->SetSizeCoefficient(0.66f)
				)
				->Split
				(
					FTabManager::NewStack()
						->AddTab(DetailsTabId, ETabState::OpenedTab)
						->SetSizeCoefficient(0.33f)
				)
		);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;

	FAssetEditorToolkit::InitAssetEditor(InMode, InToolkitHost, MediaPlayerEditorAppIdentifier, Layout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, InMediaPlayer);

	TSharedPtr<FExtender> ToolbarExtender = MakeShareable(new FExtender);

	ToolbarExtender->AddToolBarExtension(
		"Asset",
		EExtensionHook::After,
		GetToolkitCommands(),
		FToolBarExtensionDelegate::CreateStatic([](FToolBarBuilder& ToolbarBuilder, const TSharedRef<FUICommandList> ToolkitCommandsParam) {
			ToolbarBuilder.BeginSection("PlaybackControls");
			ToolbarBuilder.AddToolBarButton(FBinkMediaPlayerEditorCommands::Get().RewindMedia);
			ToolbarBuilder.AddToolBarButton(FBinkMediaPlayerEditorCommands::Get().PlayMedia);
			ToolbarBuilder.AddToolBarButton(FBinkMediaPlayerEditorCommands::Get().PauseMedia);
			ToolbarBuilder.EndSection();
		}, GetToolkitCommands())
	);

	AddToolbarExtender(ToolbarExtender);
	RegenerateMenusAndToolbars();
}

void FBinkMediaPlayerEditorToolkit::HandleEditorEndPIE(bool bIsSimulating) {
	MediaPlayer->InitializePlayer();
}

void FBinkMediaPlayerEditorToolkit::RegisterTabSpawners( const TSharedRef<class FTabManager>& TabManagerParam ) 
{
	WorkspaceMenuCategory = TabManagerParam->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_MediaPlayerEditor", "Bink Media Player Editor"));
	auto WorkspaceMenuCategoryRef = WorkspaceMenuCategory.ToSharedRef();

	FAssetEditorToolkit::RegisterTabSpawners(TabManagerParam);

	TabManagerParam->RegisterTabSpawner( ViewerTabId, FOnSpawnTab::CreateSP( this, &FBinkMediaPlayerEditorToolkit::HandleTabManagerSpawnTab, ViewerTabId ) )
		.SetDisplayName( LOCTEXT( "PlayerTabName", "Player" ) )
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon( FSlateIcon( FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Viewports" ) );

	TabManagerParam->RegisterTabSpawner( DetailsTabId, FOnSpawnTab::CreateSP( this, &FBinkMediaPlayerEditorToolkit::HandleTabManagerSpawnTab, DetailsTabId ) )
		.SetDisplayName( LOCTEXT( "DetailsTabName", "Details" ) )
		.SetGroup( WorkspaceMenuCategoryRef )
		.SetIcon( FSlateIcon( FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details" ) );
}


void FBinkMediaPlayerEditorToolkit::UnregisterTabSpawners( const TSharedRef<class FTabManager>& TabManagerParam ) 
{
	FAssetEditorToolkit::UnregisterTabSpawners(TabManagerParam);

	TabManagerParam->UnregisterTabSpawner(ViewerTabId);
	TabManagerParam->UnregisterTabSpawner(DetailsTabId);
}

TSharedRef<SDockTab> FBinkMediaPlayerEditorToolkit::HandleTabManagerSpawnTab( const FSpawnTabArgs& Args, FName TabIdentifier ) 
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;

	if (TabIdentifier == DetailsTabId) 
	{
		TabWidget = SNew(SBinkMediaPlayerEditorDetails, MediaPlayer, Style);
	} 
	else if (TabIdentifier == ViewerTabId) 
	{
		TabWidget = SNew(SBinkMediaPlayerEditorViewer, MediaPlayer, Style);
	}

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];
}


#undef LOCTEXT_NAMESPACE
