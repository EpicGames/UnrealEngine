// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "BinkMediaPlayer.h" 
#include "Toolkits/AssetEditorToolkit.h"
#include "EditorUndoClient.h"
#include "EditorReimportHandler.h"

class FSpawnTabArgs;

struct FBinkMediaPlayerEditorToolkit : public FAssetEditorToolkit, public FEditorUndoClient, public FGCObject 
{
	FBinkMediaPlayerEditorToolkit( const TSharedRef<ISlateStyle>& InStyle )
		: MediaPlayer(nullptr)
		, Style(InStyle)
	{ 
	}

	virtual ~FBinkMediaPlayerEditorToolkit() 
	{
		FReimportManager::Instance()->OnPreReimport().RemoveAll(this);
		FReimportManager::Instance()->OnPostReimport().RemoveAll(this);

		GEditor->UnregisterForUndo(this);
		FEditorDelegates::EndPIE.RemoveAll(this);
	}

	void Initialize( UBinkMediaPlayer* InMediaPlayer, const EToolkitMode::Type InMode, const TSharedPtr<class IToolkitHost>& InToolkitHost );

	// FAssetEditorToolkit interface

	virtual FString GetDocumentationLink() const override { return FString(TEXT("WorkingWithMedia/IntegratingMedia/BinkVideo")); }
	virtual void RegisterTabSpawners( const TSharedRef<class FTabManager>& TabManager ) override;
	virtual void UnregisterTabSpawners( const TSharedRef<class FTabManager>& TabManager ) override;

	// IToolkit interface

	virtual FText GetBaseToolkitName() const override { return FText::FromString(FString(TEXT("Media Asset Editor"))); }
	virtual FName GetToolkitFName() const override { return FName("MediaPlayerEditor"); }
	virtual FLinearColor GetWorldCentricTabColorScale() const override { return FLinearColor(0.3f, 0.2f, 0.5f, 0.5f); }
	virtual FString GetWorldCentricTabPrefix() const override { return FString(TEXT("BinkMediaPlayer ")); }

	// FGCObject interface

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override { Collector.AddReferencedObject(MediaPlayer); }
	virtual FString GetReferencerName() const override
	{
		return TEXT("FBinkMediaPlayerEditorToolkit");
	}

	// FEditorUndoClient interface

	virtual void PostUndo( bool bSuccess ) override { }
	virtual void PostRedo( bool bSuccess ) override { PostUndo(bSuccess); }

private:

	bool HandlePauseMediaActionCanExecute() const { return MediaPlayer->CanPause(); }
	void HandlePauseMediaActionExecute() { MediaPlayer->Pause(); }
	bool HandlePlayMediaActionCanExecute() const { return MediaPlayer->CanPlay() && (MediaPlayer->GetRate() != 1.0f); }
	void HandlePlayMediaActionExecute() { MediaPlayer->Play(); }
	bool HandleRewindMediaActionCanExecute() const { return MediaPlayer->CanPlay() && MediaPlayer->GetTime() > FTimespan::Zero(); }
	void HandleRewindMediaActionExecute() { MediaPlayer->Rewind(); }
	void HandleEditorEndPIE(bool bIsSimulating);

	TSharedRef<SDockTab> HandleTabManagerSpawnTab( const FSpawnTabArgs& Args, FName TabIdentifier );
	TObjectPtr<UBinkMediaPlayer> MediaPlayer;
	TSharedRef<ISlateStyle> Style;
};
