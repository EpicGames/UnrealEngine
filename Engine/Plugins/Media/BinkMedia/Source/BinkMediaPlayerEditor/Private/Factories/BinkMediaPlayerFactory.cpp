// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkMediaPlayerFactory.h"

#include "Misc/Paths.h"
#include "BinkMediaPlayer.h"
#include "BinkUE.h"

UBinkMediaPlayerFactory::UBinkMediaPlayerFactory( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	SupportedClass = UBinkMediaPlayer::StaticClass();

	bCreateNew = false;
	bEditorImport = true;

	Formats.Add(TEXT("bk2;Bink 2 Movie File"));
}

UObject* UBinkMediaPlayerFactory::FactoryCreateBinary( UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn ) 
{
	UBinkMediaPlayer* MediaPlayer = NewObject<UBinkMediaPlayer>(InParent, Class, Name, Flags);

	// This workaround is based on FBinkMediaPlayerCustomization::HandleUrlPickerPathPicked.
	if (CurrentFilename.IsEmpty() || CurrentFilename.StartsWith(TEXT("./")) || CurrentFilename.Contains(TEXT("://")))
	{
		MediaPlayer->OpenUrl(CurrentFilename);
	}
	else
	{
		FString FullUrl = FPaths::ConvertRelativePathToFull(CurrentFilename);
		const FString FullGameContentDir = FPaths::ConvertRelativePathToFull(BINKCONTENTPATH);
		if (FullUrl.StartsWith(FullGameContentDir))
		{
			FPaths::MakePathRelativeTo(FullUrl, *FullGameContentDir);
			FullUrl = FString(TEXT("./")) + FullUrl;
		}
		MediaPlayer->OpenUrl(FullUrl);
	}

	return MediaPlayer;
}
