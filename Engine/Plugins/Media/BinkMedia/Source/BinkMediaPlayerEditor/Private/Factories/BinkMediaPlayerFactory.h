// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "Factories/Factory.h"
#include "BinkMediaPlayerFactory.generated.h"

UCLASS(hidecategories=Object)
class UBinkMediaPlayerFactory : public UFactory 
{
	GENERATED_UCLASS_BODY()
public:
	virtual UObject* FactoryCreateBinary( UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const uint8*& Buffer, const uint8* BufferEnd, FFeedbackContext* Warn ) override;
};
