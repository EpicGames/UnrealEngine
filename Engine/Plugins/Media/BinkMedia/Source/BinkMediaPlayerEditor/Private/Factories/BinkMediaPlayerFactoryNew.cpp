// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkMediaPlayerFactoryNew.h"


UBinkMediaPlayerFactoryNew::UBinkMediaPlayerFactoryNew( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	SupportedClass = UBinkMediaPlayer::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}
