// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "Factories/Factory.h"
#include "BinkMediaPlayer.h"
#include "BinkMediaPlayerFactoryNew.generated.h"

UCLASS(hidecategories=Object)
class UBinkMediaPlayerFactoryNew : public UFactory 
{
	GENERATED_UCLASS_BODY()
public:
	virtual UObject* FactoryCreateNew( UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn ) override 
	{
		return NewObject<UBinkMediaPlayer>(InParent, InClass, InName, Flags);
	}
	virtual bool ShouldShowInNewMenu() const override 
	{ 
		return true; 
	}
};
