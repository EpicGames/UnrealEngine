// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "BinkMediaTextureFactoryNew.h"

#include "BinkMediaPlayer.h"
#include "BinkMediaTexture.h"

UBinkMediaTextureFactoryNew::UBinkMediaTextureFactoryNew( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{
	SupportedClass = UBinkMediaTexture::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UBinkMediaTextureFactoryNew::FactoryCreateNew( UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn ) 
{
	UBinkMediaTexture* MediaTexture = NewObject<UBinkMediaTexture>(InParent, InClass, InName, Flags);
	if (MediaTexture && InitialMediaPlayer) 
	{
		MediaTexture->SetMediaPlayer(InitialMediaPlayer);
	}
	return MediaTexture;
}
