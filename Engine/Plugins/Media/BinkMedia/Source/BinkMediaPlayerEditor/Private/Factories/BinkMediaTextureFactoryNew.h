// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "Factories/Factory.h"
#include "BinkMediaTextureFactoryNew.generated.h"

UCLASS(hidecategories=Object, MinimalAPI)
class UBinkMediaTextureFactoryNew : public UFactory 
{
	GENERATED_UCLASS_BODY()
public:
	UPROPERTY()
	TObjectPtr<class UBinkMediaPlayer> InitialMediaPlayer;

	virtual UObject* FactoryCreateNew( UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn ) override;
	virtual bool ShouldShowInNewMenu() const override 
	{ 
		return true; 
	}
};
