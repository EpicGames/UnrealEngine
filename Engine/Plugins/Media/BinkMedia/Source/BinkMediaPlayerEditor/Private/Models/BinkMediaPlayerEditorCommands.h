// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "Framework/Commands/Commands.h"

struct FBinkMediaPlayerEditorCommands : public TCommands<FBinkMediaPlayerEditorCommands> 
{
	FBinkMediaPlayerEditorCommands() : TCommands<FBinkMediaPlayerEditorCommands>("BinkMediaPlayerEditor", NSLOCTEXT("Contexts", "BinkMediaPlayerEditor", "Bink MediaPlayer Editor"), NAME_None, "BinkMediaPlayerEditorStyle") 
	{ 
	}

#define LOCTEXT_NAMESPACE ""
	virtual void RegisterCommands() override 
	{
		UI_COMMAND(PauseMedia, "Pause", "Pause media playback", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(PlayMedia, "Play", "Start media playback", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(RewindMedia, "Rewind", "Rewinds the media to the beginning", EUserInterfaceActionType::Button, FInputChord());
	}
#undef LOCTEXT_NAMESPACE
	
	TSharedPtr<FUICommandInfo> ForwardMedia;
	TSharedPtr<FUICommandInfo> PauseMedia;
	TSharedPtr<FUICommandInfo> PlayMedia;
	TSharedPtr<FUICommandInfo> ReverseMedia;
	TSharedPtr<FUICommandInfo> RewindMedia;
};
