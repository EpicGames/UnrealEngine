// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "BinkMediaPlayer.h"
#include "RenderingThread.h"
#include "Slate/SlateTextures.h"

struct FBinkMediaPlayerEditorTexture : public FTickableGameObject 
{
	FBinkMediaPlayerEditorTexture( FSlateTexture2DRHIRef* InSlateTexture, UBinkMediaPlayer *InMediaPlayer )
		: FTickableGameObject()
		, SlateTexture(InSlateTexture)
		, MediaPlayer(InMediaPlayer)
	{
	}

	~FBinkMediaPlayerEditorTexture() 
	{
	}

	virtual void Tick( float DeltaTime ) override 
	{
		ENQUEUE_RENDER_COMMAND(BinkEditorUpdateTexture)([MediaPlayer=MediaPlayer,SlateTexture=SlateTexture](FRHICommandListImmediate& RHICmdList) 
		{
			if (!SlateTexture->IsInitialized())
			{
				SlateTexture->InitResource(RHICmdList);
			}
			FTextureRHIRef tex = SlateTexture->GetTypedResource();
			if (!tex.GetReference())
			{
				return;
			}

			MediaPlayer->UpdateTexture(NULL, RHICmdList, tex, tex->GetNativeResource(), tex->GetSizeX(), tex->GetSizeY(), true, true, 80, 1, false, false);
		});
	}
	
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FBinkMediaPlayerEditorTexture, STATGROUP_Tickables); }
	virtual bool IsTickable() const override { return MediaPlayer != nullptr; }
	virtual bool IsTickableInEditor() const override { return true; }

	FSlateTexture2DRHIRef* SlateTexture;
	UBinkMediaPlayer *MediaPlayer;
};
