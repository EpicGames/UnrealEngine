// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#include "SBinkWidgets.h"

#include "Models/BinkMediaPlayerEditorTexture.h"
#include "Widgets/SViewport.h"
#include "DetailsViewArgs.h"
#include "IDetailsView.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SBinkMediaPlayerWidgets"

struct FBinkMediaPlayerEditorViewport : public ISlateViewport 
{
	FBinkMediaPlayerEditorViewport() : EditorTexture(), SlateTexture() 
	{ 
	}
	~FBinkMediaPlayerEditorViewport() 
	{ 
		ReleaseResources(); 
	}

	void Initialize(UBinkMediaPlayer* MediaPlayer) 
	{
		if (SlateTexture) 
		{
			ENQUEUE_RENDER_COMMAND(ResizeBinkMediaPlayerEditorViewport)([EditorTexture=EditorTexture,SlateTexture=SlateTexture,MediaPlayer=MediaPlayer](FRHICommandListImmediate& RHICmdList) 
			{ 
				const FIntPoint dim = MediaPlayer->GetDimensions();
				SlateTexture->Resize(dim.X, dim.Y);
				EditorTexture->MediaPlayer = MediaPlayer;
			});
		}
		else 
		{
			const FIntPoint dim = MediaPlayer->GetDimensions();
			EPixelFormat format = bink_force_pixel_format != PF_Unknown ? bink_force_pixel_format : GRHISupportsHDROutput ? PF_A2B10G10R10 : PF_B8G8R8A8;
			SlateTexture = new FSlateTexture2DRHIRef(dim.X, dim.Y, format, nullptr, TexCreate_RenderTargetable, true);
			EditorTexture = new FBinkMediaPlayerEditorTexture(SlateTexture, MediaPlayer);
		}
	}

	virtual FIntPoint GetSize() const override 
	{ 
		return SlateTexture ? FIntPoint(SlateTexture->GetWidth(), SlateTexture->GetHeight()) : FIntPoint(0,0); 
	}
	virtual class FSlateShaderResource* GetViewportRenderTargetTexture() const override 
	{ 
		return SlateTexture; 
	}
	virtual bool RequiresVsync() const override 
	{ 
		return false; 
	}

	void ReleaseResources() 
	{
		if (!SlateTexture) 
		{
			return; 
		}

		ENQUEUE_RENDER_COMMAND(ReleaseBinkMediaPlayerEditorResources)([SlateTexture=SlateTexture](FRHICommandListImmediate& RHICmdList) 
		{ 
				SlateTexture->ReleaseResource();
				delete SlateTexture;
		});
		delete EditorTexture;
		EditorTexture = nullptr;
		SlateTexture = nullptr;
	}

	FBinkMediaPlayerEditorTexture* EditorTexture;
	FSlateTexture2DRHIRef* SlateTexture;
};

void SBinkMediaPlayerEditorDetails::Construct( const FArguments& InArgs, UBinkMediaPlayer* InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle ) 
{
	MediaPlayer = InMediaPlayer;

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bSearchInitialKeyFocus = true;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bShowOptions = true;
	DetailsViewArgs.bShowModifiedPropertiesOption = false;

	TSharedPtr<IDetailsView> DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(MediaPlayer);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.0f, 2.0f)
			[
				DetailsView.ToSharedRef()
			]
	];
}

void SBinkMediaPlayerEditorViewer::Construct( const FArguments& InArgs, UBinkMediaPlayer* InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle ) 
{
	MediaPlayer = InMediaPlayer;
	Viewport = MakeShareable(new FBinkMediaPlayerEditorViewport());

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.0f, 0.0f)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					[
						SNew(SHorizontalBox)
							.Visibility(this, &SBinkMediaPlayerEditorViewer::HandleNoMediaSelectedTextVisibility)
						+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							[
								SNew(SImage)
									.Image(FCoreStyle::Get().GetBrush("Icons.Error"))
							]
						+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.Padding(4.0f, 0.0f, 0.0f, 0.0f)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
									.Text(LOCTEXT("NoMediaSelectedText", "Please pick a media source in the Details panel!"))
							]
					]
			]

		+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0.0f, 4.0f, 0.0f, 8.0f)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
					[
						// movie viewport
						SNew(SViewport)
							.EnableGammaCorrection(false)
							.IgnoreTextureAlpha(false)
							.EnableBlending(true)
							.PreMultipliedAlpha(false)
							.ViewportInterface(Viewport)
					]
				+ SOverlay::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Top)
					.Padding(FMargin(12.0f, 8.0f))
					[
						// playback state
						SNew(STextBlock)
							.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 20, "Regular"))
							.ShadowOffset(FVector2D(1.f, 1.f))
							.Text(this, &SBinkMediaPlayerEditorViewer::HandleOverlayStateText)
					]
			]

		+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SHorizontalBox)
				// elapsed time
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
							.Text(this, &SBinkMediaPlayerEditorViewer::HandleElapsedTimeTextBlockText)
							.ToolTipText(LOCTEXT("ElapsedTimeTooltip", "Elapsed Time"))
					]
				// scrubber
				+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.Padding(8.0f, 0.0f)
					[
						SAssignNew(ScrubberSlider, SSlider)
							.IsEnabled(this, &SBinkMediaPlayerEditorViewer::HandlePositionSliderIsEnabled)
							.OnMouseCaptureBegin(this, &SBinkMediaPlayerEditorViewer::HandlePositionSliderMouseCaptureBegin)
							.OnMouseCaptureEnd(this, &SBinkMediaPlayerEditorViewer::HandlePositionSliderMouseCaptureEnd)
							.OnValueChanged(this, &SBinkMediaPlayerEditorViewer::HandlePositionSliderValueChanged)
							.Orientation(Orient_Horizontal)
							.Value(this, &SBinkMediaPlayerEditorViewer::HandlePositionSliderValue)
					]
				// remaining time
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
							.Text(this, &SBinkMediaPlayerEditorViewer::HandleRemainingTimeTextBlockText)
							.ToolTipText(LOCTEXT("RemainingTimeTooltip", "Remaining Time"))
					]
			]
	];


	MediaPlayer->OnMediaChanged().AddRaw(this, &SBinkMediaPlayerEditorViewer::HandleMediaPlayerMediaChanged);
	RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateSP(this, &SBinkMediaPlayerEditorViewer::HandleActiveTimer));
	ReloadMediaPlayer();
}

void SBinkMediaPlayerEditorViewer::ReloadMediaPlayer() 
{
	Viewport->Initialize(MediaPlayer);
}

FText SBinkMediaPlayerEditorViewer::HandleOverlayStateText() const 
{
	if (MediaPlayer->GetUrl().IsEmpty()) 
	{
		return LOCTEXT("StateOverlayNoMedia", "No Media");
	}
	if (MediaPlayer->IsPaused()) 
	{
		return LOCTEXT("StateOverlayPaused", "Paused");
	}
	if (MediaPlayer->IsStopped()) 
	{
		return LOCTEXT("StateOverlayStopped", "Stopped");
	}
	float Rate = MediaPlayer->GetRate();
	if (FMath::IsNearlyZero(Rate) || (Rate == 1.0f)) 
	{
		return FText::GetEmpty();
	}
	if (Rate < 0.0f) 
	{
		return FText::Format(LOCTEXT("StateOverlayReverseFormat", "Reverse ({0}x)"), FText::AsNumber(-Rate));
	}
	return FText::Format(LOCTEXT("StateOverlayForwardFormat", "Forward ({0}x)"), FText::AsNumber(Rate));
}

#undef LOCTEXT_NAMESPACE
