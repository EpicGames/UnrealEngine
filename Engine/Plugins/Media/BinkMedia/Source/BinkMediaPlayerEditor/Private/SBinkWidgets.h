// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

#pragma once

#include "BinkMediaPlayer.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/SCompoundWidget.h"

struct FBinkMediaPlayerEditorViewport;

struct SBinkMediaPlayerEditorDetails : SCompoundWidget 
{
	SLATE_BEGIN_ARGS(SBinkMediaPlayerEditorDetails) 
	{ 
	}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, UBinkMediaPlayer* InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle );

	UBinkMediaPlayer* MediaPlayer;
};

struct SBinkMediaPlayerEditorViewer : SCompoundWidget 
{ 
	SLATE_BEGIN_ARGS(SBinkMediaPlayerEditorViewer) 
	{ 
	}
	SLATE_END_ARGS()

	SBinkMediaPlayerEditorViewer() : MediaPlayer() 
	{ 
	}
	~SBinkMediaPlayerEditorViewer() 
	{
		if (MediaPlayer) 
		{
			MediaPlayer->OnMediaChanged().RemoveAll(this);
		}
	}

	void Construct( const FArguments& InArgs, UBinkMediaPlayer* InMediaPlayer, const TSharedRef<ISlateStyle>& InStyle );
	void ReloadMediaPlayer();

	FText HandleOverlayStateText() const;
	FText HandleRemainingTimeTextBlockText() const 
	{ 
		return FText::AsTimespan(MediaPlayer->GetDuration() - MediaPlayer->GetTime()); 
	}

	void HandlePositionSliderMouseCaptureBegin() 
	{ 
		if (MediaPlayer->SupportsScrubbing()) 
		{ 
			PreScrubRate = MediaPlayer->GetRate(); 
			MediaPlayer->SetRate(0.0f); 
		} 
	}
	float HandlePositionSliderValue() const 
	{ 
		return MediaPlayer->GetDuration() <= FTimespan::Zero() ? 0 : (float)MediaPlayer->GetTime().GetTicks() / (float)MediaPlayer->GetDuration().GetTicks(); 
	}
	void HandlePositionSliderMouseCaptureEnd() 
	{ 
		if (MediaPlayer->SupportsScrubbing()) 
		{ 
			MediaPlayer->SetRate(PreScrubRate); 
		} 
	}

	void HandlePositionSliderValueChanged( float NewValue ) 
	{
		if (!ScrubberSlider->HasMouseCapture() || MediaPlayer->SupportsScrubbing()) 
		{
			MediaPlayer->Seek(MediaPlayer->GetDuration() * NewValue);
		}
	}

	EActiveTimerReturnType HandleActiveTimer(double InCurrentTime, float InDeltaTime) 
	{ 
		return EActiveTimerReturnType::Continue; 
	}
	FText HandleElapsedTimeTextBlockText() const 
	{ 
		return FText::AsTimespan(MediaPlayer->GetTime()); 
	}
	void HandleMediaPlayerMediaChanged() 
	{ 
		ReloadMediaPlayer(); 
	}
	EVisibility HandleNoMediaSelectedTextVisibility() const 
	{ 
		return MediaPlayer->GetUrl().IsEmpty() ? EVisibility::Visible : EVisibility::Hidden; 
	}
	bool HandlePositionSliderIsEnabled() const 
	{ 
		return MediaPlayer->SupportsSeeking(); 
	}

	UBinkMediaPlayer* MediaPlayer;
	float PreScrubRate;
	TSharedPtr<SSlider> ScrubberSlider;
	TSharedPtr<FBinkMediaPlayerEditorViewport> Viewport;
};

