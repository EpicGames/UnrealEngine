// Copyright Epic Games Tools LLC
//   Licenced under the Unreal Engine EULA 

using System.IO;
using System.Collections.Generic;
using UnrealBuildTool;

public class BinkMediaPlayerSDK : ModuleRules 
{
	// Platform Extensions need to override these
	protected virtual string LibRootDirectory { get { return ModuleDirectory; } }
	protected virtual string LibName { get { return null; } }

	protected virtual string SdkBaseDirectory { get { return LibRootDirectory; } }
	protected virtual string LibDirectory { get { return Path.Combine(SdkBaseDirectory, "lib"); } }

	protected virtual string IncDirectory { get { return Path.Combine(ModuleDirectory, "include"); } }

    public BinkMediaPlayerSDK(ReadOnlyTargetRules Target) : base(Target)
    {
		Type = ModuleType.External;

        PublicDependencyModuleNames.Add("Core");
        PublicDependencyModuleNames.Add("CoreUObject");
        PublicDependencyModuleNames.Add("Engine");
        PublicDependencyModuleNames.Add("InputCore");
        PublicDependencyModuleNames.Add("RenderCore");
        PublicDependencyModuleNames.Add("RHI");
        PublicDependencyModuleNames.Add("MoviePlayer");
        //PublicDependencyModuleNames.Add("MediaAssets");
        PublicDependencyModuleNames.Add("Projects");

        PrivatePCHHeaderFile = "Private/BinkMediaPlayerPCH.h";

        if (Target.bBuildEditor == true)
        {
			PublicDependencyModuleNames.Add("Slate");
			PublicDependencyModuleNames.Add("SlateCore");
			PublicDependencyModuleNames.Add("DesktopWidgets");
			PublicDefinitions.Add("BINKPLUGIN_UE4_EDITOR=1");
            PrivateDependencyModuleNames.AddRange(new string[] { "PropertyEditor", "DesktopPlatform", "SourceControl", "EditorStyle", "UnrealEd" });
        }
        else
        {
            PublicDefinitions.Add("BINKPLUGIN_UE4_EDITOR=0");
        }

		PublicDefinitions.Add("BUILDING_FOR_UNREAL_ONLY=1");
		PublicDefinitions.Add("__RADNOEXPORTS__=1");
		PublicDefinitions.Add("__RADINSTATICLIB__=1");
		//RuntimeDependencies.Add("$(ProjectDir)/Content/Movies/..."); // For chunked streaming

		string Lib = LibName;
		string Platform = Target.Platform.ToString();

		if(Lib == null)
		{
			if (Target.IsInPlatformGroup(UnrealPlatformGroup.Microsoft))
			{
				if (Target.Architecture == UnrealArch.Arm64)
				{
					Lib = "BinkUnrealWinArm64.lib";
				}
				else
				{
					Lib = "BinkUnreal" + Platform + ".lib";
				}
			}
			else if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				Lib = "BinkUnreal" + Platform + ".a";
				PublicDependencyModuleNames.Add("MetalRHI");
				PublicFrameworks.Add("AudioToolbox");
			}
			else if (Target.Platform == UnrealTargetPlatform.IOS)
			{
				Lib = "libBinkUnreal" + Platform + ".a";
				PublicDependencyModuleNames.Add("MetalRHI");
			}
			else if (Target.Platform == UnrealTargetPlatform.TVOS || Target.Platform == UnrealTargetPlatform.VisionOS)
			{
				Lib = null; // Unsupported platform
				PublicDependencyModuleNames.Add("MetalRHI");
			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Apple))
			{
				Lib = "BinkUnreal" + Platform + ".a";
				PublicDependencyModuleNames.Add("MetalRHI");
			}
			else if (Target.Platform == UnrealTargetPlatform.Android)
			{
				PublicDependencyModuleNames.Add("Launch");
				PublicAdditionalLibraries.Add(Path.Combine(LibDirectory, "android", "armv7", "libBinkUnrealAndroid.a"));
				PublicAdditionalLibraries.Add(Path.Combine(LibDirectory, "android", "arm64", "libBinkUnrealAndroid.a"));
				PublicAdditionalLibraries.Add(Path.Combine(LibDirectory, "android", "x86", "libBinkUnrealAndroid.a"));
				PublicAdditionalLibraries.Add(Path.Combine(LibDirectory, "android", "x64", "libBinkUnrealAndroid.a"));
			}
			else
			{
				Lib = "BinkUnreal" + Platform + ".a";
			}
		}

		if (Lib != null)
		{
			PublicAdditionalLibraries.Add(Path.Combine(LibDirectory, Lib));
		}
        PublicSystemIncludePaths.Add(IncDirectory);
	}
}
