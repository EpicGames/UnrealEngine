// Copyright Epic Games, Inc. All Rights Reserved.
// Licensed under the terms of a valid Unreal Engine license agreement,
//   and the separate 'Unreal Engine End User License Agreement for Publishing'.

// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __BINKH__
#define __BINKH__

typedef struct BINK * HBINK;

typedef struct BINK 
{
  U32 Width;                  // Width (1 based, 640 for example)
  U32 Height;                 // Height (1 based, 480 for example)
  U32 Frames;                 // Number of frames (1 based, 100 = 100 frames)
  U32 FrameNum;               // Frame to *be* displayed (1 based)

  U32 LastFrameNum;           // Last frame decompressed or skipped (1 based)
  U32 FrameRate;              // Frame Rate Numerator
  U32 FrameRateDiv;           // Frame Rate Divisor (frame rate=numerator/divisor)
  U32 ReadError;              // Non-zero if a read error has ocurred

  U32 OpenFlags;              // flags used on open
  U32 BinkType;               // Bink flags
  U32 LargestFrameSize;       // Largest frame size
  U32 FrameSize;              // The current frame's size in bytes

  U32 SndSize;                // The current frame sound tracks' size in bytes
  U32 FrameChangePercent;     // very rough percentage of the frame that changed
  S32 NumTracks;              // how many tracks

} BINK;

// ================================================================================
// Bink API to open and search for offsets within a zip file (play bink in zip if bink is uncompressed)

RADEXPFUNC void * RADEXPLINK BinkZipOpen( char const * zipfn );
RADEXPFUNC S32 RADEXPLINK BinkZipFindUncompressedFileOffset( void * zip, char const * filename, U64 * out_ofs );
RADEXPFUNC S32 RADEXPLINK BinkZipGetFileInfo( void * zip, U64 index, char * out_filename, U32 bytes_filename, U64 * out_offset, S32 * out_uncompressed );
RADEXPFUNC void RADEXPLINK BinkZipClose( void * zip );


// ================================================================================
// Bink high level API (usually used by Unity)

RADEXPFUNC void RADEXPLINK BinkHLInit( U32 thread_count, U32 snd_freq, U32 snd_chans ); // snd_freq == 0, no sound init
RADEXPFUNC void RADEXPLINK BinkHLShutdown( void );

#define BINKHLSNDLAYOUT_NONE               0 // don't open any sound tracks snd_track_start not used
#define BINKHLSNDLAYOUT_SIMPLE             1 // based on filename, OR simply mono or stereo sound in track snd_track_start (default speaker spread)
#define BINKHLSNDLAYOUT_LANGUAGEOVERRIDE   2 // mono or stereo sound in track 0, language track at snd_track_start
#define BINKHLSNDLAYOUT_51                 3 // 6 mono tracks in tracks snd_track_start[0..5]
#define BINKHLSNDLAYOUT_51LANGUAGEOVERRIDE 4 // 6 mono tracks in tracks 0..5, center language track at snd_track_start
#define BINKHLSNDLAYOUT_71                 5 // 8 mono tracks in tracks snd_track_start[0..7]
#define BINKHLSNDLAYOUT_71LANGUAGEOVERRIDE 6 // 8 mono tracks in tracks 0..7, center language track at snd_track_start

#define BINKHLBUFFER_STREAM              0 // stream the movie off the media during playback (caches about 1 second of video)
#define BINKHLBUFFER_LOADALL             1 // loads the whole movie into memory at Open time (will block)
#define BINKHLBUFFER_STREAMUNTILRESIDENT 2 // streams the movie into a memory buffer as big as the movie, so it will be preloaded eventually)


RADEXPFUNC HBINK RADEXPLINK BinkHLOpen( char const * name, U32 snd_track_type, U32 snd_track_start, U32 buffering_type, U64 file_byte_offset );

typedef struct BINKHLPRELOADED BINKHLPRELOADED;
RADEXPFUNC BINKHLPRELOADED * RADEXPLINK BinkHLPreload( char const * name, U64 file_byte_offset );
RADEXPFUNC void RADEXPLINK BinkHLUnpreload( BINKHLPRELOADED * pre );
RADEXPFUNC HBINK RADEXPLINK BinkHLOpenPreload( BINKHLPRELOADED * pre, U32 snd_track_type, U32 snd_track_start );

typedef struct BINKHLINFO
{
  U32 Width;
  U32 Height;
  U32 TextureBufferYAWidth;
  U32 TextureBufferYAHeight;
  U32 TextureBufferChWidth;
  U32 TextureBufferChHeight;
  U32 TextureDrawIndex;
  U32 TextureDrawFrame;     // what frame would be drawn with TextureDrawIndex
  S32 NeedAlpha;
  S32 NeedHDR;
  U32 Frames;
  U32 FrameNum;
  U32 FrameRate;
  U32 FrameRateDiv;
  U32 LoopsRemaining;
  U32 ReadError;
  U32 NumTracksOpened;
  U32 PlaybackState;         // 0 = playing, 1 = paused, 2 = gotoing, 3 = at end (stopped)
  F32 cs0, cs1, cs2, cs3, cs4, cs5, cs6, cs7, cs8, cs9, cs10, cs11, cs12, cs13, cs14, cs15;
} BINKHLINFO;

RADEXPFUNC void RADEXPLINK BinkHLGetInfo( HBINK b, BINKHLINFO * info );

RADEXPFUNC void RADEXPLINK BinkHLRegisterTextureBuffers( HBINK b, U8 ** Luma,  // two luma pointers
                                                                  U8 ** cR,    // two chroma pointers
                                                                  U8 ** cB, 
                                                                  U8 ** Alpha,
                                                                  U8 ** HDR );

RADEXPFUNC void RADEXPLINK BinkHLPause( HBINK b, U32 on_off );

RADEXPFUNC void RADEXPLINK BinkHLVolume( HBINK b, F32 vol );

// set speaker volumes
//   BinkSndSimple = count must be 2 (l/r) - OR match filename (is there a "_51." or "_51L." or "_71." or "_71L." in the filename?)
//   BinkSndLanguageOverride = count must be 3 (l,r)/language
//   BinkSnd51 = count must be 6 (front l/r),center,sub,(rear l/r),
//   BinkSnd51LanguageOverride = 7 (front l/r),center,sub,(rear l/r),language
//   BinkSnd71 = count must be 8 (front l/r),center,sub,(read l/r),(surr l/r)
//   BinkSnd71LanguageOverride = 9 (front l/r),center,sub,(read l/r),(surr l/r), lang
RADEXPFUNC void RADEXPLINK BinkHLSpeakerVolumes( HBINK b, F32 * vols, U32 count );

RADEXPFUNC void RADEXPLINK BinkHLLoopCount( HBINK b, U32 count );  // 0 = loop forever

RADEXPFUNC void RADEXPLINK BinkHLGoto( HBINK b, U32 goto_frame, U32 max_frames_to_decode_per_process );  // goto frame goes from 1 to framenum (0 means stop current goto), max_frames_to_decode_per_process = 0 means block until at frame

RADEXPFUNC void RADEXPLINK BinkHLProcess( void ); // decodes/moves to next frame (next frame, or loop, or goto) for all Binks that are playing (if they need advancing)

RADEXPFUNC char * RADEXPLINK BinkGetError( void );

RADEXPFUNC void RADEXPLINK BinkClose( HBINK bnk );

#define BINKSURFACE32BGRx      3
#define BINKSURFACE32RGBx      4
#define BINKSURFACE32BGRA      5
#define BINKSURFACE32RGBA      6
#define BINKSURFACE32ARGB     12

RADEXPFUNC S32  RADEXPLINK BinkCopyToBuffer(HBINK bnk,void* dest,S32 destpitch,U32 destheight,U32 destx,U32 desty,U32 flags);
RADEXPFUNC S32  RADEXPLINK BinkCopyToBufferRect(HBINK bnk,void* dest,S32 destpitch,U32 destheight,U32 destx,U32 desty,U32 srcx, U32 srcy, U32 srcw, U32 srch, U32 flags);

RADEXPFUNC S32 RADEXPLINK BinkLoadSubtitles( HBINK bink, char const * srt_file );
RADEXPFUNC char const * RADEXPLINK BinkCurrentSubtitle( HBINK bink, U32 * iterate, U32 * opt_out_start_ms, U32 * opt_out_end_ms );
RADEXPFUNC char const * RADEXPLINK BinkGetSubtitleByIndex( HBINK bink, U32 index, U32 * opt_out_start_ms, U32 * opt_out_end_ms );

#endif
